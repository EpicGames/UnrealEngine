// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class InterchangeTestEditor : ModuleRules
	{
		public InterchangeTestEditor(ReadOnlyTargetRules Target) : base(Target)
		{
            PublicDependencyModuleNames.AddRange(
				new string[]
				{
                    "Core",
					"CoreUObject",
					"ContentBrowser",
					"Engine",
					"EditorStyle",
					"InputCore",
					"InterchangeCore",
					"InterchangeTests",
					"LevelEditor",
					"PropertyEditor",
					"ToolMenus",
					"UnrealEd"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"AssetTools",
					"WorkspaceMenuStructure",
					"SlateCore",
					"Slate",
					"AutomationTest"
				}
			);
		}
    }
}
