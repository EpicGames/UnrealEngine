// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_InterchangeImportTestPlan.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_InterchangeImportTestPlan)

TConstArrayView<FAssetCategoryPath> UAssetDefinition_InterchangeImportTestPlan::GetAssetCategories() const
{
	static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Misc ,NSLOCTEXT("AssetDefinition", "InterchangeImportTestPlan_SubMenu", "Other"), ECategoryMenuType::Section)
		};
	return Categories;
}

EAssetCommandResult UAssetDefinition_InterchangeImportTestPlan::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UInterchangeImportTestPlan* TestPlan : OpenArgs.LoadObjects<UInterchangeImportTestPlan>())
	{
		FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, TestPlan);
	}

	return EAssetCommandResult::Handled;
}
