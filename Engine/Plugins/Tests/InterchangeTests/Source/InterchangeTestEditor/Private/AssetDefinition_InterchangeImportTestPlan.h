// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetTypeActions_Base.h"
#include "InterchangeImportTestPlan.h"

#include "AssetDefinition_InterchangeImportTestPlan.generated.h"

UCLASS()
class UAssetDefinition_InterchangeImportTestPlan : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions_InterchangeImportTestPlan", "DisplayName", "Interchange Import Test Plan"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(125, 0, 200); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UInterchangeImportTestPlan::StaticClass(); }
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual bool CanImport() const override { return false; }
	// UAssetDefinition End
};
