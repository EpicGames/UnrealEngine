// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeImportTestPlanAssetDetails.h"
#include "InterchangeImportTestPlan.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#include "AutomationTestExcludelist.h"
#include "AssetRegistry/AssetData.h"

#define LOCTEXT_NAMESPACE "InterchangeImportTestPlanAssetDetails"

FInterchangeImportTestPlanAssetDetailsCustomization::FInterchangeImportTestPlanAssetDetailsCustomization()
	: InterchangeImportTestPlan(nullptr)
	, CachedDetailBuilder(nullptr)
{
}

TSharedRef<IDetailCustomization> FInterchangeImportTestPlanAssetDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FInterchangeImportTestPlanAssetDetailsCustomization);
}

void FInterchangeImportTestPlanAssetDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	CachedDetailBuilder = &DetailBuilder;
	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailBuilder.GetObjectsBeingCustomized(EditingObjects);
	
	if (!ensure(EditingObjects.Num() == 1))
	{
		return;
	}

	InterchangeImportTestPlan = Cast<UInterchangeImportTestPlan>(EditingObjects[0].Get());

	if (!ensure(InterchangeImportTestPlan.IsValid()))
	{
		return;
	}

	CustomizeAutomationCategory();
}

void FInterchangeImportTestPlanAssetDetailsCustomization::CustomizeAutomationCategory()
{
	TFunction<bool(const UInterchangeImportTestPlan* InTestPlanAsset, FString& OutReason)> IsTestPlanAssetSkipped = [](const UInterchangeImportTestPlan* InTestPlanAsset, FString& OutReason)
		{
			if (!InTestPlanAsset)
			{
				return false;
			}

			if (UAutomationTestExcludelist* AutomationTestExcludeList = UAutomationTestExcludelist::Get())
			{
				FAssetData TestPlanAssetData(InTestPlanAsset);
				constexpr bool bAddBeautifiedTestNamePrefix = true;
				const FString TestName = UE::Interchange::FInterchangeImportTestPlanStaticHelpers::GetTestNameFromObjectPathString(TestPlanAssetData.GetObjectPathString(), bAddBeautifiedTestNamePrefix);
				if (const FAutomationTestExcludelistEntry* TestEntry = AutomationTestExcludeList->GetExcludeTestEntry(TestName))
				{
					OutReason = TestEntry->Reason.ToString();
					return true;
				}
			}

			return false;
		};


	IDetailCategoryBuilder& AutomationCategory = CachedDetailBuilder->EditCategory(FName("Automation"), FText::GetEmpty(), ECategoryPriority::Uncommon);
	FDetailWidgetRow& IsTestSkippedRow = AutomationCategory.AddCustomRow(LOCTEXT("IsTestSkippedFilterString", "Is Test Skipped"));
	IsTestSkippedRow
		.NameContent()
		[
			SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text(LOCTEXT("IsTestSkipped_TextBlockText", "Is Test Skipped"))
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
						.IsEnabled(false)
						.IsChecked_Lambda([this, IsTestPlanAssetSkipped]() {
						if (InterchangeImportTestPlan.IsValid())
						{
							FString OutReason;
							if (IsTestPlanAssetSkipped(InterchangeImportTestPlan.Get(), OutReason))
							{
								return ECheckBoxState::Checked;
							}
						}

						return ECheckBoxState::Unchecked;
							})
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0, 4.0)
				[
					SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						.Visibility_Lambda([this, IsTestPlanAssetSkipped]()
							{
								if (InterchangeImportTestPlan.IsValid())
								{
									FString OutReason;
									if (IsTestPlanAssetSkipped(InterchangeImportTestPlan.Get(), OutReason))
									{
										return EVisibility::Visible;
									}
								}

								return EVisibility::Collapsed;
							})
						.Text_Lambda([this, IsTestPlanAssetSkipped]()
							{
								if (InterchangeImportTestPlan.IsValid())
								{
									FString OutReason;
									if (IsTestPlanAssetSkipped(InterchangeImportTestPlan.Get(), OutReason))
									{
										return FText::FromString(FString::Printf(TEXT("Reason Skipped: %s"), *OutReason));
									}
								}
								return FText::GetEmpty();
							})
				]
		];
}

#undef LOCTEXT_NAMESPACE