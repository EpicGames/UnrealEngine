// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/SharedPointer.h"

class UInterchangeImportTestPlan;

class FInterchangeImportTestPlanAssetDetailsCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	/** End IDetailCustomization interface */

private:
	FInterchangeImportTestPlanAssetDetailsCustomization();

	void CustomizeAutomationCategory();
private:
	TWeakObjectPtr<UInterchangeImportTestPlan> InterchangeImportTestPlan;

	IDetailLayoutBuilder* CachedDetailBuilder;	// The detail builder for this customisation
};