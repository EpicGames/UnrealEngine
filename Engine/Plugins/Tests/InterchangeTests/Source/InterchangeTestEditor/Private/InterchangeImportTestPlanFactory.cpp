// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeImportTestPlanFactory.h"
#include "InterchangeImportTestPlan.h"
#include "AssetTypeCategories.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeImportTestPlanFactory)

#define LOCTEXT_NAMESPACE "InterchangeImportTestPlanFactory"


UInterchangeImportTestPlanFactory::UInterchangeImportTestPlanFactory()
{
	bCreateNew = true;
	bEditAfterNew = false;
	bEditorImport = false;
	SupportedClass = UInterchangeImportTestPlan::StaticClass();
}


UObject* UInterchangeImportTestPlanFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* Context, FFeedbackContext* Warn)
{
	UInterchangeImportTestPlan* TestPlan = NewObject<UInterchangeImportTestPlan>(InParent, SupportedClass, InName, InFlags | RF_Transactional);
	return TestPlan;
}


FText UInterchangeImportTestPlanFactory::GetDisplayName() const
{
	return LOCTEXT("MenuEntry", "Interchange Import Test Plan");
}


#undef LOCTEXT_NAMESPACE

