// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "InterchangeImportTestPlanFactory.generated.h"


UCLASS(hidecategories=Object, collapsecategories)
class UInterchangeImportTestPlanFactory : public UFactory
{
	GENERATED_BODY()

public:
	UInterchangeImportTestPlanFactory();

	// UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FText GetDisplayName() const override;
	// End of UFactory Interface
};
