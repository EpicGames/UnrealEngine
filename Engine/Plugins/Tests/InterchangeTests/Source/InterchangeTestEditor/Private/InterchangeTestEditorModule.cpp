// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeTestEditorModule.h"
#include "InterchangeTestFunctionLayout.h"
#include "InterchangeTestFunction.h"
#include "InterchangeTestPlanPipelineSettings.h"
#include "InterchangeTestPlanPipelineSettingsLayout.h"
#include "InterchangeImportTestPlanAssetDetails.h"
#include "InterchangeImportTestPlan.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"


#define LOCTEXT_NAMESPACE "InterchangeTestEditorModule"


FInterchangeTestEditorModule& FInterchangeTestEditorModule::Get()
{
	return FModuleManager::LoadModuleChecked<FInterchangeTestEditorModule>(INTERCHANGETESTEDITOR_MODULE_NAME);
}


bool FInterchangeTestEditorModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(INTERCHANGETESTEDITOR_MODULE_NAME);
}


void FInterchangeTestEditorModule::StartupModule()
{
	// Register the FInterchangeTestFunction struct customization
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomClassLayout(UInterchangeImportTestPlan::StaticClass()->GetFName(), 
		FOnGetDetailCustomizationInstance::CreateStatic(&FInterchangeImportTestPlanAssetDetailsCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(FInterchangeTestFunction::StaticStruct()->GetFName(),
	FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FInterchangeTestFunctionLayout::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(FInterchangeTestPlanPipelineSettings::StaticStruct()->GetFName(),
	FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FInterchangeTestPlanPipelineSettingsLayout::MakeInstance));

	PropertyModule.NotifyCustomizationModuleChanged();
}


void FInterchangeTestEditorModule::ShutdownModule()
{
	if (UObjectInitialized())
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout(UInterchangeImportTestPlan::StaticClass()->GetFName());
		PropertyModule.UnregisterCustomPropertyTypeLayout(FInterchangeTestFunction::StaticStruct()->GetFName());
		PropertyModule.UnregisterCustomPropertyTypeLayout(FInterchangeTestPlanPipelineSettings::StaticStruct()->GetFName());
	}
}


IMPLEMENT_MODULE(FInterchangeTestEditorModule, InterchangeTestEditor);


#undef LOCTEXT_NAMESPACE

