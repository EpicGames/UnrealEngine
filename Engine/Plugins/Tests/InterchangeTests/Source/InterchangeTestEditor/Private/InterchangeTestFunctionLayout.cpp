// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeTestFunctionLayout.h"
#include "Framework/Views/TableViewMetadata.h"
#include "InterchangeTestFunction.h"

#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboBox.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "InterchangeTestFunctionLayout"

TSharedRef<IPropertyTypeCustomization> FInterchangeTestFunctionLayout::MakeInstance()
{
	return MakeShared<FInterchangeTestFunctionLayout>();
}


FInterchangeTestFunctionLayout::FInterchangeTestFunctionLayout()
{
}


FInterchangeTestFunctionLayout::~FInterchangeTestFunctionLayout()
{
	FEditorDelegates::PostUndoRedo.RemoveAll(this);
}


void FInterchangeTestFunctionLayout::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Perform initialization here rather than in the constructor
	// (adding SP delegates doesn't work if the object hasn't yet been assigned)

	AssetClasses = FInterchangeTestFunction::GetAvailableAssetClasses();
	FEditorDelegates::PostUndoRedo.AddSP(this, &FInterchangeTestFunctionLayout::RefreshLayout);

	// Ensure that the details layout is refreshed when resetting the entire struct to default, so that custom data can be rebuilt

	StructProperty = StructPropertyHandle;
	StructProperty->SetOnPropertyResetToDefault(FSimpleDelegate::CreateSP(this, &FInterchangeTestFunctionLayout::RefreshLayout));

	// Cache all the child properties we're interested in manipulating

	AssetClassProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInterchangeTestFunction, AssetClass));
	OptionalAssetNameProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInterchangeTestFunction, OptionalAssetName));
	CheckFunctionProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInterchangeTestFunction, CheckFunction));
	ParametersProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInterchangeTestFunction, Parameters));
	PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();

	// The FInterchangeTestFunction defines a specific test which will be carried out on the imported asset.
	// The three important properties are:
	// 1) AssetClass - the class of the asset object we are expecting, and which we will perform our test on.
	// 2) TestFunction - a UFunction* specifying the function we will call to perform the test
	// 3) Parameters - a list of specific parameter values to call the function with
	//
	// The Parameters property is a simple (name, valueString) map.
	// However this is not a useful form for interacting with in a properties view, so we import the parameter values to binary here.
	// Note that this binary blob is owned by, and only of interest to, this layout class.

	FInterchangeTestFunction* InterchangeTestFunction = GetStruct();
	GetFunctionsForClass(InterchangeTestFunction->AssetClass);
	ParamData = InterchangeTestFunction->ImportParameters();

	// The header of the struct customization contains the class and function combo boxes.
	// These will determine which children should be populated underneath.

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SComboBox<UClass*>)
				.OptionsSource(&AssetClasses)
				.OnGenerateWidget(this, &FInterchangeTestFunctionLayout::OnGenerateClassComboWidget)
				.OnSelectionChanged(this, &FInterchangeTestFunctionLayout::OnClassComboSelectionChanged)
				.InitiallySelectedItem(InterchangeTestFunction->AssetClass)
				[
					OnGenerateClassComboWidget(InterchangeTestFunction->AssetClass)
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SEditableTextBox)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.HintText(LOCTEXT("OptionalAssetName", "Optional Asset Name"))
				.Text_Lambda([this] { FString Value; OptionalAssetNameProperty->GetValue(Value); return FText::FromString(Value); })
				.OnTextCommitted_Lambda([this](const FText& Val, ETextCommit::Type TextCommitType) { OptionalAssetNameProperty->SetValue(Val.ToString()); })
				.ToolTipText(LOCTEXT("OptionalAssetNameTooltip", "Fill out this field with the name of an imported asset which you wish to test, if there are multiple assets of the same type to choose from."))
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SComboBox<UFunction*>)
				.OptionsSource(&Functions)
				.OnGenerateWidget(this, &FInterchangeTestFunctionLayout::OnGenerateFunctionComboWidget)
				.OnSelectionChanged(this, &FInterchangeTestFunctionLayout::OnFunctionComboSelectionChanged)
				.InitiallySelectedItem(InterchangeTestFunction->CheckFunction)
				[
					OnGenerateFunctionComboWidget(InterchangeTestFunction->CheckFunction)
				]
			]
		]
	];
}


void FInterchangeTestFunctionLayout::GetFunctionsForClass(UClass* AssetClass)
{
	// This populates an array with all the UFunction* available as tests for the given asset class
	Functions = FInterchangeTestFunction::GetAvailableFunctions(AssetClass);
}


void FInterchangeTestFunctionLayout::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// The children of the struct customization represent all the editable input parameters to bind to the test function
	// This needs to be rebuilt every time a new function is selected.

	FInterchangeTestFunction* InterchangeTestFunction = GetStruct();

	// Iterate through the parameters of interest
	for (FProperty* ParamProperty : InterchangeTestFunction->GetParameters())
	{
		FName ParamName = ParamProperty->GetFName();
		IDetailPropertyRow* Row = StructBuilder.AddExternalStructureProperty(ParamData.ToSharedRef(), ParamName, FAddPropertyParams().ForceShowProperty());
		TSharedPtr<IPropertyHandle> PropertyHandle = Row->GetPropertyHandle();

		// Note: we need to hook both of these events.
		// SetOnPropertyValueChanged is called when the property itself is modified.
		// SetOnChildPropertyValueChanged is called when a child property is modified, e.g. the component of a vector.

		PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FInterchangeTestFunctionLayout::OnParameterChanged, PropertyHandle));
		PropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FInterchangeTestFunctionLayout::OnParameterChanged, PropertyHandle));

		// We have a notion of parameter defaults from the C++ function prototype.
		// Hook a custom handler for Reset To Default for these parameter properties so that it does something useful.

		Row->OverrideResetToDefault(FResetToDefaultOverride::Create(FSimpleDelegate::CreateSP(this, &FInterchangeTestFunctionLayout::OnParameterResetToDefault, PropertyHandle)));
	}
}


FInterchangeTestFunction* FInterchangeTestFunctionLayout::GetStruct() const
{
	// Get address of the FInterchangeTestFunction struct being viewed.
	// We only ever expect the property handle to be linked to a single instance.

	void* StructPtr = nullptr;
	StructProperty->GetValueData(StructPtr);
	return static_cast<FInterchangeTestFunction*>(StructPtr);
}


void FInterchangeTestFunctionLayout::OnParameterChanged(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	// When we change a function parameter property in the property editor, this will change the binary copy we maintain here.
	// Here we need to propagate the change to the struct's (name, value) map.

	FString Value;
	PropertyHandle->GetValueAsFormattedString(Value);
	FName PropertyName = PropertyHandle->GetProperty()->GetFName();

	void* MapPtr = nullptr;
	ParametersProperty->GetValueData(MapPtr);

	// Note that we are not accessing the map property directly through the handle, so we need to call these various notifies ourselves.
	// This allows transactions to work correctly.

	ParametersProperty->NotifyPreChange();
	static_cast<decltype(FInterchangeTestFunction::Parameters)*>(MapPtr)->Add(PropertyName, Value);
	ParametersProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	ParametersProperty->NotifyFinishedChangingProperties();
}


void FInterchangeTestFunctionLayout::OnParameterResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	// The property editor doesn't start a transaction if ResetToDefault is custom. We have to do it ourselves!
	FScopedTransaction Transaction(FText::Format(LOCTEXT("ResetParameterToDefault", "Reset {0} to Default"), PropertyHandle->GetPropertyDisplayName()));

	FName PropertyName = PropertyHandle->GetProperty()->GetFName();

	// The way we implement ResetToDefault is by just removing the parameter from the map completely.
	// (note the requirement to call the notifies manually)
	// The default value will be set in the binary copy we maintain when the parameter is imported.

	void* MapPtr = nullptr;
	ParametersProperty->GetValueData(MapPtr);

	ParametersProperty->NotifyPreChange();
	static_cast<decltype(FInterchangeTestFunction::Parameters)*>(MapPtr)->Remove(PropertyName);
	ParametersProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	ParametersProperty->NotifyFinishedChangingProperties();

	FInterchangeTestFunction* InterchangeTestFunction = GetStruct();
	InterchangeTestFunction->ImportParameter(*ParamData, PropertyHandle->GetProperty());
}


void FInterchangeTestFunctionLayout::RefreshLayout()
{
	if (PropertyUtilities.IsValid())
	{
		PropertyUtilities->ForceRefresh();
	}
}


TSharedRef<SWidget> FInterchangeTestFunctionLayout::OnGenerateClassComboWidget(UClass* InClass)
{
	return SNew(SBox)
	[
		SNew(STextBlock)
		.Text(InClass ? InClass->GetDisplayNameText() : LOCTEXT("SelectAssetClass", "<Expected asset class>"))
		.ToolTipText(InClass ? InClass->GetToolTipText() : FText())
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];
}


void FInterchangeTestFunctionLayout::OnClassComboSelectionChanged(UClass* InSelectedItem, ESelectInfo::Type SelectInfo)
{
	FInterchangeTestFunction* InterchangeTestFunction = GetStruct();

	if (InterchangeTestFunction->AssetClass != InSelectedItem)
	{
		// When the asset class combo box of the header changes, this changes the entire state of the struct.
		// The TestFunction needs to be reset (as the old one is most likely unsuitable), and so does the stored parameter map.
		// So wrap this entire operation in a single high-level transaction, and force the layout to be refreshed afterwards.

		FScopedTransaction Transaction(LOCTEXT("EditTestType", "Change test to perform"));
		AssetClassProperty->SetValue(InSelectedItem);
		UFunction* NullFunction = nullptr;
		CheckFunctionProperty->SetValue(NullFunction);
		ParametersProperty->AsMap()->Empty();

		RefreshLayout();
	}
}


TSharedRef<SWidget> FInterchangeTestFunctionLayout::OnGenerateFunctionComboWidget(UFunction* InFunction)
{
	FInterchangeTestFunction* InterchangeTestFunction = GetStruct();

	// If a test function doesn't have the correct signature, it will appear in the list, but greyed out and unselectable.
	// This will act as a signal to the developer to fix the incorrect signature.
	return SNew(SBox)
	[
		SNew(STextBlock)
		.Text(InFunction ? InFunction->GetDisplayNameText() : LOCTEXT("SelectTest", "<Please select a test>"))
		.ToolTipText(InFunction ? InFunction->GetToolTipText() : FText())
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.IsEnabled(InterchangeTestFunction->IsValid(InFunction))
	];
}


void FInterchangeTestFunctionLayout::OnFunctionComboSelectionChanged(UFunction* InSelectedItem, ESelectInfo::Type SelectInfo)
{
	FInterchangeTestFunction* InterchangeTestFunction = GetStruct();

	if (InterchangeTestFunction->IsValid(InSelectedItem) && InterchangeTestFunction->CheckFunction != InSelectedItem)
	{
		// When the function combo box of the header changes, we need to repopulate the parameter map.
		// So wrap this entire operation in a single high-level transaction, and force the layout to be refreshed afterwards.

		FScopedTransaction Transaction(LOCTEXT("EditTestType", "Change test to perform"));
		CheckFunctionProperty->SetValue(InSelectedItem);
		ParametersProperty->AsMap()->Empty();

		RefreshLayout();
	}
}

#undef LOCTEXT_NAMESPACE
