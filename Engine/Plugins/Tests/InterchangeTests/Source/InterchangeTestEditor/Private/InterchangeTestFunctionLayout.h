// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class FStructOnScope;
class SWidget;
namespace ESelectInfo { enum Type : int; }

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyUtilities;
struct FInterchangeTestFunction;


class FInterchangeTestFunctionLayout : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FInterchangeTestFunctionLayout();
	virtual ~FInterchangeTestFunctionLayout();

	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	FInterchangeTestFunction* GetStruct() const;

	void GetFunctionsForClass(UClass* Class);
	TSharedRef<SWidget> OnGenerateClassComboWidget(UClass* InClass);
	void OnClassComboSelectionChanged(UClass* InSelectedItem, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> OnGenerateFunctionComboWidget(UFunction* InFunction);
	void OnFunctionComboSelectionChanged(UFunction* InSelectedItem, ESelectInfo::Type SelectInfo);
	void OnParameterChanged(TSharedPtr<IPropertyHandle> PropertyHandle);
	void OnParameterResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle);
	void RefreshLayout();

	/** List of all the asset classes supported by the testing framework */
	TArray<UClass*> AssetClasses;

	/** List of all the test functions supported by the current asset class */
	TArray<UFunction*> Functions;

	/** Bound parameter data stored as a binary blob */
	TSharedPtr<FStructOnScope> ParamData;

	TSharedPtr<IPropertyHandle> StructProperty;
	TSharedPtr<IPropertyHandle> AssetClassProperty;
	TSharedPtr<IPropertyHandle> OptionalAssetNameProperty;
	TSharedPtr<IPropertyHandle> CheckFunctionProperty;
	TSharedPtr<IPropertyHandle> ParametersProperty;

	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};
