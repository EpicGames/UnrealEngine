// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeTestPlanPipelineSettingsLayout.h"
#include "InterchangeTestPlanPipelineSettings.h"
#include "InterchangeImportTestStepBase.h"
#include "InterchangeImportTestStepImport.h"
#include "InterchangeImportTestStepReimport.h"
#include "InterchangePipelineBase.h"

#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "HAL/IConsoleManager.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "InterchangeTestPipelineSettings"

static bool GInterchangeTestPlanCanEditCustomPipelines = false;
static FAutoConsoleVariableRef CCvarInterchangeDefaultShowEssentialsView(
	TEXT("Interchange.TestPlan.CanEditCustomPipelines"),
	GInterchangeTestPlanCanEditCustomPipelines,
	TEXT("Can the Pipelines in the Test Plan asset be edited."),
	ECVF_Default);


/////////// FInterchangeTestPlanPipelineSettingsLayout ///////////

TSharedRef<IPropertyTypeCustomization> FInterchangeTestPlanPipelineSettingsLayout::MakeInstance()
{
	return MakeShared<FInterchangeTestPlanPipelineSettingsLayout>();
}

FInterchangeTestPlanPipelineSettingsLayout::FInterchangeTestPlanPipelineSettingsLayout()
{
}

FInterchangeTestPlanPipelineSettingsLayout::~FInterchangeTestPlanPipelineSettingsLayout()
{
	FEditorDelegates::PostUndoRedo.RemoveAll(this);
}

void FInterchangeTestPlanPipelineSettingsLayout::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FEditorDelegates::PostUndoRedo.AddSP(this, &FInterchangeTestPlanPipelineSettingsLayout::RefreshLayout);

	StructProperty = StructPropertyHandle;
	PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
					.Text_Lambda([this]() 
						{
							return LOCTEXT("PipelineSettingsLayout_EditDefaults", "Edit Pipeline Settings");
						})
					.IsEnabled_Lambda([this]() 
						{
							if (FInterchangeTestPlanPipelineSettings* PipelineData = GetStruct())
							{
								return PipelineData->CanEditPipelineSettings();
							}

							return false;
						})
					.OnClicked(this, &FInterchangeTestPlanPipelineSettingsLayout::EditPipelineSettings)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
					.Text_Lambda([this]()
						{
							return LOCTEXT("PipelineSettingsLayout_Clear", "Clear");
						})
					.OnClicked(this, &FInterchangeTestPlanPipelineSettingsLayout::ClearModifiedPipelineSettings)
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0, 2.0)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
					.Visibility_Lambda([this]() 
					{
						if (FInterchangeTestPlanPipelineSettings* PipelineSettings = GetStruct())
						{
							if (UInterchangeImportTestStepBase* TestStep = PipelineSettings->ParentTestStep.Get())
							{
								constexpr bool bCheckForValidPipelines = false;
								if (!TestStep->IsUsingOverridePipelines(bCheckForValidPipelines))
								{
									return EVisibility::Visible;
								}
							}
						}
						return EVisibility::Collapsed;
					})
					.Text_Lambda([this]()
					{
						if (FInterchangeTestPlanPipelineSettings* PipelineSettings = GetStruct())
						{
							if (PipelineSettings->CustomPipelines.IsEmpty())
							{
								return LOCTEXT("PipelineSettingsCountEmptyText", "Pipeline Count : Empty");
							}

							return FText::Format(LOCTEXT("PipelineSettingsCountText", "Pipeline Count : {0}"), PipelineSettings->CustomPipelines.Num());
						}

						return FText::GetEmpty();
					})
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
	];
}

void FInterchangeTestPlanPipelineSettingsLayout::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!StructPropertyHandle->IsValidHandle())
	{
		return;
	}

	uint32 NumChildren = 0;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FInterchangeTestPlanPipelineSettings, CustomPipelines))
		{
			// Only show the property in the editor when the CVar is enabled.
			if (GInterchangeTestPlanCanEditCustomPipelines)
			{
				StructBuilder.AddProperty(ChildHandle);
			}
		}
		else 
		{
			StructBuilder.AddProperty(ChildHandle);
		}
	}
}

void FInterchangeTestPlanPipelineSettingsLayout::RefreshLayout()
{
	if (PropertyUtilities.IsValid())
	{
		PropertyUtilities->ForceRefresh();
	}
}

FReply FInterchangeTestPlanPipelineSettingsLayout::EditPipelineSettings()
{
	if (FInterchangeTestPlanPipelineSettings* PipelineSettings = GetStruct())
	{
		if (UInterchangeImportTestStepBase* TestStep = PipelineSettings->ParentTestStep.Get())
		{
			TestStep->EditPipelineSettings();
		}
	}
	return FReply::Handled();
}

FReply FInterchangeTestPlanPipelineSettingsLayout::ClearModifiedPipelineSettings()
{
	if (FInterchangeTestPlanPipelineSettings* PipelineSettings = GetStruct())
	{
		if (UInterchangeImportTestStepBase* TestStep = PipelineSettings->ParentTestStep.Get())
		{
			TestStep->ClearPipelineSettings();
		}
	}
	return FReply::Handled();
}

FInterchangeTestPlanPipelineSettings* FInterchangeTestPlanPipelineSettingsLayout::GetStruct() const
{
	// Get address of the FInterchangeTestPlanPipelineSettings struct being viewed.
	// We only ever expect the property handle to be linked to a single instance.

	void* StructPtr = nullptr;
	StructProperty->GetValueData(StructPtr);
	return static_cast<FInterchangeTestPlanPipelineSettings*>(StructPtr);
}

#undef LOCTEXT_NAMESPACE 