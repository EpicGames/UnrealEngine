// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "IPropertyTypeCustomization.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyUtilities;
struct FInterchangeTestPlanPipelineSettings;
class FReply;

class FInterchangeTestPlanPipelineSettingsLayout : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FInterchangeTestPlanPipelineSettingsLayout();
	virtual ~FInterchangeTestPlanPipelineSettingsLayout();

	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	void RefreshLayout();

	FReply EditPipelineSettings();
	FReply ClearModifiedPipelineSettings();

private:
	FInterchangeTestPlanPipelineSettings* GetStruct() const;

private:
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
	TSharedPtr<IPropertyHandle> StructProperty;

};