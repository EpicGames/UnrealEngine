// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class FAssetTypeActions_InterchangeImportTestPlan;


#define INTERCHANGETESTEDITOR_MODULE_NAME TEXT("InterchangeTestEditor")

/**
 * Module for implementing the editor for Interchange automation tests
 */
class FInterchangeTestEditorModule : public IModuleInterface
{
public:
	static FInterchangeTestEditorModule& Get();
	static bool IsAvailable();

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	TSharedPtr<FAssetTypeActions_InterchangeImportTestPlan> AssetTypeActions;
};
