// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class InterchangeTests : ModuleRules
	{
		public InterchangeTests(ReadOnlyTargetRules Target) : base(Target)
		{
            PublicDependencyModuleNames.AddRange(
				new string[]
				{
                    "Core",
					"CoreUObject",
					"Engine",
					"Json",
					"JsonUtilities"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"InterchangePipelines",
					"InterchangeCore",
					"InterchangeDispatcher",
					"InterchangeEngine",
					"LevelSequence",
					"MeshDescription",
					"MovieScene",
					"StaticMeshDescription",
					"SkeletalMeshDescription",
					"UnrealEd",
					"VariantManagerContent",
					"FunctionalTesting",
					"LevelEditor"
				}
			);
		}
    }
}
