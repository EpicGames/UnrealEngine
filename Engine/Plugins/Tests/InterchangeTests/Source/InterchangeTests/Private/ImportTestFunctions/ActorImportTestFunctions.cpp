// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/ActorImportTestFunctions.h"
#include "GameFramework/Actor.h"
#include "ImportTestFunctions/ImportTestFunctionsBase.h"
#include "InterchangeTestFunction.h"
#include "Internationalization/Regex.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorImportTestFunctions)

namespace UE::Interchange::Tests::Private
{
	bool TryGetPropertyTextValue(const UObject* Object, const FString& PropertyName, FString& OutTextValue)
	{
		const FProperty* Property = Object->GetClass()->FindPropertyByName(*PropertyName);
		if (Property == nullptr)
		{
			return false;
		}

		Property->ExportTextItem_InContainer(OutTextValue, Object, nullptr, nullptr, PPF_None);
		return true;
	}

	const UActorComponent* FindActorComponentByName(const AActor* Actor, const FName& ComponentName)
	{
		for (const UActorComponent* Component : Actor->GetComponents())
		{
			if (Component->GetFName() == ComponentName)
			{
				return Component;
			}
		}

		return nullptr;
	}
}

UClass* UActorImportTestFunctions::GetAssociatedAssetType() const
{
	return AActor::StaticClass();
}

FInterchangeTestFunctionResult UActorImportTestFunctions::CheckImportedActorCount(const TArray<AActor*>& Actors, int32 ExpectedNumberOfImportedActors)
{
	FInterchangeTestFunctionResult Result;
	if (Actors.Num() != ExpectedNumberOfImportedActors)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d actors, imported %d."), ExpectedNumberOfImportedActors, Actors.Num()));
	}

	return Result;
}

FInterchangeTestFunctionResult UActorImportTestFunctions::CheckActorClassCount(const TArray<AActor*>& Actors, TSubclassOf<AActor> Class, int32 ExpectedNumberOfActors)
{
	FInterchangeTestFunctionResult Result;
	const int32 ImportedNumberOfActors = Actors.FilterByPredicate([Class](const AActor* Actor){ return Actor->GetClass() == Class; }).Num();

	if (ImportedNumberOfActors != ExpectedNumberOfActors)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d actors of class %s, imported %d."), ExpectedNumberOfActors, *Class->GetName(), ImportedNumberOfActors));
	}

	return Result;
}

FInterchangeTestFunctionResult UActorImportTestFunctions::CheckActorClass(const AActor* Actor, TSubclassOf<AActor> ExpectedClass)
{
	FInterchangeTestFunctionResult Result;
	const UClass* ImportedClass = Actor->GetClass();

	if (ImportedClass != ExpectedClass)
	{
		Result.AddError(FString::Printf(TEXT("Expected actor of class %s, imported %s."), *ExpectedClass->GetName(), *ImportedClass->GetName()));
	}

	return Result;
}

FInterchangeTestFunctionResult UActorImportTestFunctions::CheckActorPropertyValue(const AActor* Actor, const FString& PropertyName, bool bUseRegexToMatchValue, const FString& ExpectedValue)
{
	using namespace UE::Interchange::Tests::Private;

	FInterchangeTestFunctionResult Result;
	FString ImportedTextValue;

	if (!TryGetPropertyTextValue(Actor, PropertyName, ImportedTextValue))
	{
		Result.AddError(FString::Printf(TEXT("The imported actor doesn't contain property '%s'."), *PropertyName));
	}
	else
	{
		if (bUseRegexToMatchValue)
		{
			FRegexMatcher RegexMatcher(FRegexPattern(ExpectedValue), ImportedTextValue);
			if (!RegexMatcher.FindNext())
			{
				Result.AddError(FString::Printf(TEXT("For actor property '%s', expected value to match regex pattern '%s', imported '%s'."), *PropertyName, *ExpectedValue, *ImportedTextValue));
			}
		}
		else if (ImportedTextValue != ExpectedValue)
		{
			Result.AddError(FString::Printf(TEXT("For actor property '%s', expected value '%s', imported '%s'."), *PropertyName, *ExpectedValue, *ImportedTextValue));
		}
	}

	return Result;
}

FInterchangeTestFunctionResult UActorImportTestFunctions::CheckComponentPropertyValue(const AActor* Actor, const FString& ComponentName, const FString& PropertyName, bool bUseRegexToMatchValue, const FString& ExpectedValue)
{
	using namespace UE::Interchange::Tests::Private;

	FInterchangeTestFunctionResult Result;
	const UActorComponent* Component = FindActorComponentByName(Actor, *ComponentName);

	if (Component == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported actor doesn't contain component '%s'."), *ComponentName));
	}
	else
	{
		FString ImportedTextValue;
		if (!TryGetPropertyTextValue(Component, PropertyName, ImportedTextValue))
		{
			Result.AddError(FString::Printf(TEXT("The imported actor component '%s' doesn't contain property '%s'."), *ComponentName, *PropertyName));
		}
		else
		{
			if (bUseRegexToMatchValue)
			{
				FRegexMatcher RegexMatcher(FRegexPattern(ExpectedValue), ImportedTextValue);
				if (!RegexMatcher.FindNext())
				{
					Result.AddError(FString::Printf(TEXT("For property '%s' in actor component '%s', expected value to match regex pattern '%s', imported '%s'."), *PropertyName, *ComponentName, *ExpectedValue, *ImportedTextValue));
				}
			}
			else if (ImportedTextValue != ExpectedValue)
			{
				Result.AddError(FString::Printf(TEXT("For property '%s' in actor component '%s', expected value '%s', imported '%s'."), *PropertyName, *ComponentName, *ExpectedValue, *ImportedTextValue));
			}
		}
	}

	return Result;
}
