// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/AnimationImportTestFunctions.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "ImportTestFunctions/ImportTestFunctionsBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationImportTestFunctions)

namespace UE::Private::AnimationImportTestFunction
{
	static bool GetImportedCustomCurveKey(UAnimSequence* AnimSequence, const FString& CurveName, const int32 KeyIndex, FRichCurveKey& OutCurveKey, FInterchangeTestFunctionResult& Result)
	{
		const FFloatCurve* FloatCurve = nullptr;

		for (const FFloatCurve& Curve : AnimSequence->GetCurveData().FloatCurves)
		{
			if (Curve.GetName() == FName(*CurveName))
			{
				FloatCurve = &Curve;
				break;
			}
		}

		if (FloatCurve == nullptr)
		{
			Result.AddError(FString::Printf(TEXT("No custom curve named %s was imported"), *CurveName));
			return false;
		}

		const FRichCurve& RichCurve = FloatCurve->FloatCurve;
		const TArray<FRichCurveKey>& Keys = RichCurve.GetConstRefOfKeys();
		if (!Keys.IsValidIndex(KeyIndex))
		{
			Result.AddError(FString::Printf(TEXT("No key at the index %d was imported in curve %s"), KeyIndex, *CurveName));
			return false;
		}

		OutCurveKey = Keys[KeyIndex];
		return true;
	}
}

UClass* UAnimationImportTestFunctions::GetAssociatedAssetType() const
{
	return UAnimSequence::StaticClass();
}

FInterchangeTestFunctionResult UAnimationImportTestFunctions::CheckImportedAnimSequenceCount(const TArray<UAnimSequence*>& AnimSequences, int32 ExpectedNumberOfImportedAnimSequences)
{
	FInterchangeTestFunctionResult Result;
	if (AnimSequences.Num() != ExpectedNumberOfImportedAnimSequences)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d anim sequences, imported %d."), ExpectedNumberOfImportedAnimSequences, AnimSequences.Num()));
	}

	return Result;
}

FInterchangeTestFunctionResult UAnimationImportTestFunctions::CheckAnimationLength(UAnimSequence* AnimSequence, float ExpectedAnimationLength)
{
	FInterchangeTestFunctionResult Result;

	float animationLength = AnimSequence->GetPlayLength();
	if (!FMath::IsNearlyEqual(animationLength, ExpectedAnimationLength, UE_KINDA_SMALL_NUMBER))
	{
		Result.AddError(FString::Printf(TEXT("Expected animation length of %f seconds, imported %f."), ExpectedAnimationLength, animationLength));
	}

	return Result;
}

FInterchangeTestFunctionResult UAnimationImportTestFunctions::CheckAnimationFrameNumber(UAnimSequence* AnimSequence, int32 ExpectedFrameNumber)
{
	FInterchangeTestFunctionResult Result;

	int32 FrameNumber = AnimSequence->GetNumberOfSampledKeys();
	if (FrameNumber != ExpectedFrameNumber)
	{
		Result.AddError(FString::Printf(TEXT("Expected animation frame number %d, imported %d."), ExpectedFrameNumber, FrameNumber));
	}

	return Result;
}

FInterchangeTestFunctionResult UAnimationImportTestFunctions::CheckCurveKeyTime(UAnimSequence* AnimSequence, const FString& CurveName, int32 KeyIndex, float ExpectedCurveKeyTime)
{
	FInterchangeTestFunctionResult Result;
	FRichCurveKey CurveKey;
	if (UE::Private::AnimationImportTestFunction::GetImportedCustomCurveKey(AnimSequence, CurveName, KeyIndex, CurveKey, Result))
	{
		if (!FMath::IsNearlyEqual(CurveKey.Time, ExpectedCurveKeyTime, UE_KINDA_SMALL_NUMBER))
		{
			Result.AddError(FString::Printf(TEXT("Expected animation curve key time %f, imported %f."), ExpectedCurveKeyTime, CurveKey.Time));
		}
	}
	return Result;
}

FInterchangeTestFunctionResult UAnimationImportTestFunctions::CheckCurveKeyValue(UAnimSequence* AnimSequence, const FString& CurveName, int32 KeyIndex, float ExpectedCurveKeyValue)
{
	FInterchangeTestFunctionResult Result;
	FRichCurveKey CurveKey;
	if (UE::Private::AnimationImportTestFunction::GetImportedCustomCurveKey(AnimSequence, CurveName, KeyIndex, CurveKey, Result))
	{
		if (!FMath::IsNearlyEqual(CurveKey.Value, ExpectedCurveKeyValue, UE_KINDA_SMALL_NUMBER))
		{
			Result.AddError(FString::Printf(TEXT("Expected animation curve key value %f, imported %f."), ExpectedCurveKeyValue, CurveKey.Value));
		}
	}
	return Result;
}

FInterchangeTestFunctionResult UAnimationImportTestFunctions::CheckCurveKeyArriveTangent(UAnimSequence* AnimSequence, const FString& CurveName, int32 KeyIndex, float ExpectedCurveKeyArriveTangent)
{
	FInterchangeTestFunctionResult Result;
	FRichCurveKey CurveKey;
	if (UE::Private::AnimationImportTestFunction::GetImportedCustomCurveKey(AnimSequence, CurveName, KeyIndex, CurveKey, Result))
	{
		if (!FMath::IsNearlyEqual(CurveKey.ArriveTangent, ExpectedCurveKeyArriveTangent, UE_KINDA_SMALL_NUMBER))
		{
			Result.AddError(FString::Printf(TEXT("Expected animation curve key arrive tangent %f, imported %f."), ExpectedCurveKeyArriveTangent, CurveKey.ArriveTangent));
		}
	}
	return Result;
}

FInterchangeTestFunctionResult UAnimationImportTestFunctions::CheckCurveKeyArriveTangentWeight(UAnimSequence* AnimSequence, const FString& CurveName, int32 KeyIndex, float ExpectedCurveKeyArriveTangentWeight)
{
	FInterchangeTestFunctionResult Result;
	FRichCurveKey CurveKey;
	if (UE::Private::AnimationImportTestFunction::GetImportedCustomCurveKey(AnimSequence, CurveName, KeyIndex, CurveKey, Result))
	{
		if (!FMath::IsNearlyEqual(CurveKey.ArriveTangentWeight, ExpectedCurveKeyArriveTangentWeight, UE_KINDA_SMALL_NUMBER))
		{
			Result.AddError(FString::Printf(TEXT("Expected animation curve key arrive tangent weight %f, imported %f."), ExpectedCurveKeyArriveTangentWeight, CurveKey.ArriveTangentWeight));
		}
	}
	return Result;
}

FInterchangeTestFunctionResult UAnimationImportTestFunctions::CheckCurveKeyLeaveTangent(UAnimSequence* AnimSequence, const FString& CurveName, int32 KeyIndex, float ExpectedCurveKeyLeaveTangent)
{
	FInterchangeTestFunctionResult Result;
	FRichCurveKey CurveKey;
	if (UE::Private::AnimationImportTestFunction::GetImportedCustomCurveKey(AnimSequence, CurveName, KeyIndex, CurveKey, Result))
	{
		if (!FMath::IsNearlyEqual(CurveKey.LeaveTangent, ExpectedCurveKeyLeaveTangent, UE_KINDA_SMALL_NUMBER))
		{
			Result.AddError(FString::Printf(TEXT("Expected animation curve key Leave tangent %f, imported %f."), ExpectedCurveKeyLeaveTangent, CurveKey.LeaveTangent));
		}
	}
	return Result;
}

FInterchangeTestFunctionResult UAnimationImportTestFunctions::CheckCurveKeyLeaveTangentWeight(UAnimSequence* AnimSequence, const FString& CurveName, int32 KeyIndex, float ExpectedCurveKeyLeaveTangentWeight)
{
	FInterchangeTestFunctionResult Result;
	FRichCurveKey CurveKey;
	if (UE::Private::AnimationImportTestFunction::GetImportedCustomCurveKey(AnimSequence, CurveName, KeyIndex, CurveKey, Result))
	{
		if (!FMath::IsNearlyEqual(CurveKey.LeaveTangentWeight, ExpectedCurveKeyLeaveTangentWeight, UE_KINDA_SMALL_NUMBER))
		{
			Result.AddError(FString::Printf(TEXT("Expected animation curve key Leave tangent weight %f, imported %f."), ExpectedCurveKeyLeaveTangentWeight, CurveKey.LeaveTangentWeight));
		}
	}
	return Result;
}
