// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/AssetImportTestFunctions.h"

#include "AssetRegistry/AssetData.h"
#include "ImportTestFunctions/ImportTestFunctionsBase.h"
#include "UObject/MetaData.h"
#include "InterchangeTestFunction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetImportTestFunctions)

UClass* UAssetImportTestFunctions::GetAssociatedAssetType() const
{
	return UObject::StaticClass();
}

FInterchangeTestFunctionResult UAssetImportTestFunctions::CheckImportedMetadataCount(const UObject* Object, const int32 ExpectedNumberOfMetadataForThisObject)
{
	FInterchangeTestFunctionResult Result;
	int32 ObjectMetadataCount = 0;
	if (const TMap<FName, FString>* ObjectMetaDatas = FMetaData::GetMapForObject(Object))
	{
		ObjectMetadataCount = ObjectMetaDatas->Num();
	}
	if (ObjectMetadataCount != ExpectedNumberOfMetadataForThisObject)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d object metadatas, imported %d."), ExpectedNumberOfMetadataForThisObject, ObjectMetadataCount));
	}
	return Result;
}

FInterchangeTestFunctionResult UAssetImportTestFunctions::CheckMetadataExist(const UObject* Object, const FString& ExpectedMetadataKey)
{
	FInterchangeTestFunctionResult Result;
	bool bMetadatExist = false;
	if (const TMap<FName, FString>* ObjectMetaDatas = FMetaData::GetMapForObject(Object))
	{
		bMetadatExist = ObjectMetaDatas->Contains(FName(*ExpectedMetadataKey));
	}
	if (!bMetadatExist)
	{
		Result.AddError(FString::Printf(TEXT("Expected object metadata key %s was not imported."), *ExpectedMetadataKey));
	}
	return Result;
}

FInterchangeTestFunctionResult UAssetImportTestFunctions::CheckMetadataValue(const UObject* Object, const FString& ExpectedMetadataKey, const FString& ExpectedMetadataValue)
{
	FInterchangeTestFunctionResult Result;
	bool bKeyExist = false;
	if (const TMap<FName, FString>* ObjectMetaDatas = FMetaData::GetMapForObject(Object))
	{
		if (const FString* MetadataValue = ObjectMetaDatas->Find(FName(*ExpectedMetadataKey)))
		{
			bKeyExist = true;
			if (!ExpectedMetadataValue.Equals(*MetadataValue))
			{
				Result.AddError(FString::Printf(TEXT("Expected object metadata key [%s] value [%s], found a different value [%s]."), *ExpectedMetadataKey, *ExpectedMetadataValue, *(*MetadataValue)));
			}
		}
	}
	
	if(!bKeyExist)
	{
		Result.AddError(FString::Printf(TEXT("Expected object metadata key [%s] value [%s], the key was not imported."), *ExpectedMetadataKey, *ExpectedMetadataValue));
	}
	return Result;
}

FInterchangeTestFunctionResult UAssetImportTestFunctions::CheckObjectPathHasSubstring(const UObject* Object, const FString& ExpectedPathString)
{
	FInterchangeTestFunctionResult Result;
	if (Object)
	{
		FAssetData ObjectAssetData(Object);
		const FString ObjectPathString = ObjectAssetData.GetObjectPathString();
		const int32 FoundIndex = ObjectPathString.Find(ExpectedPathString);
		if (FoundIndex < 0)
		{
			Result.AddError(FString::Printf(TEXT("Expected Object Path to contain [%s], but no matching substring was found in the object path [%s]. "), *ExpectedPathString, *ObjectPathString));
		}
	}
	else
	{
		Result.AddError(TEXT("Can't retrieve paths invalid object."));
	}
	return Result;
}


