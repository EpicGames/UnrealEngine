// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/InterchangeResultImportTestFunctions.h"
#include "ImportTestFunctions/ImportTestFunctionsBase.h"
#include "InterchangeResultsContainer.h"
#include "InterchangeTestFunction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeResultImportTestFunctions)


UClass* UInterchangeResultImportTestFunctions::GetAssociatedAssetType() const
{
	return UInterchangeResultsContainer::StaticClass();
}


FInterchangeTestFunctionResult UInterchangeResultImportTestFunctions::CheckIfErrorOrWarningWasGenerated(UInterchangeResultsContainer* ResultsContainer, TSubclassOf<UInterchangeResult> ErrorOrWarningClass, const FString& Message)
{
	FInterchangeTestFunctionResult Result;

	bool bFoundType = false;
	for (UInterchangeResult* ResultObject : ResultsContainer->GetResults())
	{
		if (ResultObject->IsA(ErrorOrWarningClass))
		{
			bFoundType = true;
			// If we matched the message type, remove it from the results container so it won't flag as an error at the end of the test.
			// Note: we can do this while iterating through results, because UInterchangeResultsContainer::GetResults passes the array by value,
			// for thread-safety purposes.
			ResultsContainer->RemoveResult(ResultObject);

			const FString ResultMessage = ResultObject->GetText().ToString();
			if (!Message.IsEmpty() && ResultMessage != Message)
			{
				Result.AddError(FString::Printf(TEXT("Expected '%s' error message text, found '%s'."), *Message, *ResultMessage));
			}
		}
	}

	if (!bFoundType)
	{
		if(ErrorOrWarningClass)
		{
			Result.AddError(FString::Printf(TEXT("No error or warning of class %s was generated."), *ErrorOrWarningClass.Get()->GetName()));
		}
		else
		{
			Result.AddError(FString::Printf(TEXT("No error or warning class set.")));
		}
	}

	return Result;
}

