// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/LevelSequenceImportTestFunctions.h"
#include "InterchangeTestFunction.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequenceImportTestFunctions)

namespace UE::Interchange::Tests::Private
{
	bool TryGetSectionInterpolationMode(const UMovieSceneSection* Section, ERichCurveInterpMode& OutInterpMode)
	{
		bool HasInterpMode = false;
		const FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();

		for (FMovieSceneFloatChannel* Channel : ChannelProxy.GetChannels<FMovieSceneFloatChannel>())
		{
			for (const FMovieSceneFloatValue& Value : Channel->GetData().GetValues())
			{
				if (!HasInterpMode)
				{
					HasInterpMode = true;
					OutInterpMode = Value.InterpMode;
				}
				else if (Value.InterpMode != OutInterpMode)
				{
					return false;
				}
			}
		}

		for (FMovieSceneDoubleChannel* Channel : ChannelProxy.GetChannels<FMovieSceneDoubleChannel>())
		{
			for (const FMovieSceneDoubleValue& Value : Channel->GetData().GetValues())
			{
				if (!HasInterpMode)
				{
					HasInterpMode = true;
					OutInterpMode = Value.InterpMode;
				}
				else if (Value.InterpMode != OutInterpMode)
				{
					return false;
				}
			}
		}

		return HasInterpMode;
	}
}

UClass* ULevelSequenceImportTestFunctions::GetAssociatedAssetType() const
{
	return ULevelSequence::StaticClass();
}

FInterchangeTestFunctionResult ULevelSequenceImportTestFunctions::CheckLevelSequenceCount(const TArray<ULevelSequence*>& LevelSequences, int32 ExpectedNumberOfLevelSequences)
{
	FInterchangeTestFunctionResult Result;
	if (LevelSequences.Num() != ExpectedNumberOfLevelSequences)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d level sequences, imported %d."), ExpectedNumberOfLevelSequences, LevelSequences.Num()));
	}

	return Result;
}

FInterchangeTestFunctionResult ULevelSequenceImportTestFunctions::CheckSequenceLength(const ULevelSequence* LevelSequence, float ExpectedLevelSequenceLength)
{
	FInterchangeTestFunctionResult Result;
	const TObjectPtr<UMovieScene> MovieScene = LevelSequence->GetMovieScene();

	if (MovieScene == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported level sequence doesn't contain a valid movie scene")));
	}
	else
	{
		const float ImportedLevelSequenceLength = MovieScene->GetTickResolution().AsSeconds(MovieScene->GetPlaybackRange().Size<FFrameNumber>());
		if (!FMath::IsNearlyEqual(ImportedLevelSequenceLength, ExpectedLevelSequenceLength, UE_KINDA_SMALL_NUMBER))
		{
			Result.AddError(FString::Printf(TEXT("Expected level sequence length of %f seconds, imported %f."), ExpectedLevelSequenceLength, ImportedLevelSequenceLength));
		}
	}

	return Result;
}

FInterchangeTestFunctionResult ULevelSequenceImportTestFunctions::CheckSectionCount(const ULevelSequence* LevelSequence, int32 ExpectedNumberOfSections)
{
	FInterchangeTestFunctionResult Result;
	const TObjectPtr<UMovieScene> MovieScene = LevelSequence->GetMovieScene();

	if (MovieScene == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported level sequence doesn't contain a valid movie scene")));
	}
	else
	{
		const int32 ImportedNumberOfSections = MovieScene->GetAllSections().Num();
		if (ImportedNumberOfSections != ExpectedNumberOfSections)
		{
			Result.AddError(FString::Printf(TEXT("Expected %d sections in level sequence, imported %d."), ExpectedNumberOfSections, ImportedNumberOfSections));
		}
	}

	return Result;
}

FInterchangeTestFunctionResult ULevelSequenceImportTestFunctions::CheckSectionInterpolationMode(const ULevelSequence* LevelSequence, int32 SectionIndex, ERichCurveInterpMode ExpectedInterpolationMode)
{
	using namespace UE::Interchange::Tests::Private;

	FInterchangeTestFunctionResult Result;
	const TObjectPtr<UMovieScene> MovieScene = LevelSequence->GetMovieScene();

	if (MovieScene == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported level sequence doesn't contain a valid movie scene")));
	}
	else
	{
		const TArray<UMovieSceneSection*> Sections = MovieScene->GetAllSections();
		if (!Sections.IsValidIndex(SectionIndex))
		{
			Result.AddError(FString::Printf(TEXT("The imported level sequence doesn't contain section %d."), SectionIndex));
		}
		else
		{
			ERichCurveInterpMode ImportedInterpolationMode;
			if (!TryGetSectionInterpolationMode(Sections[SectionIndex], ImportedInterpolationMode))
			{
				Result.AddError(FString::Printf(TEXT("The imported level sequence doesn't have a consistent interpolation mode for section %d."), SectionIndex));
			}
			else if (ImportedInterpolationMode != ExpectedInterpolationMode)
			{
				const FString ImportedDisplayValue = UEnum::GetDisplayValueAsText(ImportedInterpolationMode).ToString();
				const FString ExpectedDisplayValue = UEnum::GetDisplayValueAsText(ExpectedInterpolationMode).ToString();
				Result.AddError(FString::Printf(TEXT("For section %d, expected interpolation mode %s, imported %s."), SectionIndex, *ExpectedDisplayValue, *ImportedDisplayValue));
			}
		}
	}

	return Result;
}
