// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/LevelVariantSetsImportTestFunctions.h"
#include "LevelVariantSets.h"
#include "ImportTestFunctions/ImportTestFunctionsBase.h"
#include "InterchangeTestFunction.h"
#include "Variant.h"
#include "VariantSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelVariantSetsImportTestFunctions)


UClass* ULevelVariantSetsImportTestFunctions::GetAssociatedAssetType() const
{
	return ULevelVariantSets::StaticClass();
}

FInterchangeTestFunctionResult ULevelVariantSetsImportTestFunctions::CheckLevelVariantSetsCount(const TArray<ULevelVariantSets*>& LevelVariantSetsAssets, int32 ExpectedNumberOfLevelVariantSets)
{
	FInterchangeTestFunctionResult Result;
	if (LevelVariantSetsAssets.Num() != ExpectedNumberOfLevelVariantSets)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d level variant sets, imported %d."), ExpectedNumberOfLevelVariantSets, LevelVariantSetsAssets.Num()));
	}

	return Result;
}

FInterchangeTestFunctionResult ULevelVariantSetsImportTestFunctions::CheckVariantSetsCount(ULevelVariantSets* LevelVariantSets, int32 ExpectedNumberOfVariantSets)
{
	FInterchangeTestFunctionResult Result;
	const int32 ImportedNumberOfVariantSets = LevelVariantSets->GetNumVariantSets();

	if (ImportedNumberOfVariantSets!= ExpectedNumberOfVariantSets)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d variant sets, imported %d."), ExpectedNumberOfVariantSets, ImportedNumberOfVariantSets));
	}

	return Result;
}

FInterchangeTestFunctionResult ULevelVariantSetsImportTestFunctions::CheckVariantsCount(ULevelVariantSets* LevelVariantSets, const FString& VariantSetName, int32 ExpectedNumberOfVariants)
{
	FInterchangeTestFunctionResult Result;
	const UVariantSet* VariantSet = LevelVariantSets->GetVariantSetByName(VariantSetName);

	if (VariantSet == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported level variant sets doesn't contain variant set '%s'."), *VariantSetName));
	}
	else
	{
		const int32 ImportedNumberOfVariants = VariantSet->GetNumVariants();
		if (ImportedNumberOfVariants != ExpectedNumberOfVariants)
		{
			Result.AddError(FString::Printf(TEXT("For variant set '%s', expected %d variants, imported %d."), *VariantSetName, ExpectedNumberOfVariants, ImportedNumberOfVariants));
		}
	}

	return Result;
}

FInterchangeTestFunctionResult ULevelVariantSetsImportTestFunctions::CheckBindingsCount(ULevelVariantSets* LevelVariantSets, const FString& VariantSetName, const FString& VariantName, int32 ExpectedNumberOfBindings)
{
	FInterchangeTestFunctionResult Result;
	UVariantSet* VariantSet = LevelVariantSets->GetVariantSetByName(VariantSetName);

	if (VariantSet == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported level variant sets doesn't contain variant set '%s'."), *VariantSetName));
	}
	else
	{
		const UVariant* Variant = VariantSet->GetVariantByName(VariantName);
		if (Variant == nullptr)
		{
			Result.AddError(FString::Printf(TEXT("The imported variant set '%s' doesn't contain variant '%s'."), *VariantSetName, *VariantName));
		}
		else
		{
			const int32 ImportedNumberOfBindings = Variant->GetBindings().Num();
			if (ImportedNumberOfBindings != ExpectedNumberOfBindings)
			{
				Result.AddError(FString::Printf(TEXT("For variant '%s' in set '%s', expected %d bindings, imported %d."), *VariantName, *VariantSetName, ExpectedNumberOfBindings, ImportedNumberOfBindings));
			}
		}
	}

	return Result;
}
