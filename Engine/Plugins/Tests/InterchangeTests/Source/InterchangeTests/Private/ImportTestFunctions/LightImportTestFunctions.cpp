// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/LightImportTestFunctions.h"
#include "Components/LightComponent.h"
#include "Engine/Light.h"
#include "InterchangeTestsMathUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LightImportTestFunctions)

UClass* ULightImportTestFunctions::GetAssociatedAssetType() const
{
	return ALight::StaticClass();
}

FInterchangeTestFunctionResult ULightImportTestFunctions::CheckLightPosition(ALight* Light, const FVector& ExpectedLightPosition)
{
	FInterchangeTestFunctionResult Result;
	
	if(ULightComponent* LightComponent = Light->GetLightComponent())
	{
		using namespace InterchangeTestsMathUtilities;
		const FVector ExpectedLightPositionRounded = RoundVectorToDecimalPlaces(ExpectedLightPosition);
		if(const FVector LightPosition = RoundVectorToDecimalPlaces(FVector(LightComponent->GetLightPosition())); 
			!LightPosition.Equals(ExpectedLightPositionRounded))
		{
			Result.AddError(FString::Printf(TEXT("Expected (%g, %g, %g) light position, imported (%g, %g, %g)."),
				ExpectedLightPositionRounded.X, ExpectedLightPositionRounded.Y, ExpectedLightPositionRounded.Z,
				LightPosition.X, LightPosition.Y, LightPosition.Z));
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("LightComponent is null for %s."), *Light->GetName()));
	}

	return Result;
}

FInterchangeTestFunctionResult ULightImportTestFunctions::CheckLightDirection(ALight* Light, const FVector& ExpectedLightDirection)
{
	FInterchangeTestFunctionResult Result;

	if(ULightComponent* LightComponent = Light->GetLightComponent())
	{
		using namespace InterchangeTestsMathUtilities;
		const FVector ExpectedLightDirectionRounded = RoundVectorToDecimalPlaces(ExpectedLightDirection);
		if(const FVector LightDirection = RoundVectorToDecimalPlaces(LightComponent->GetDirection()); !LightDirection.Equals(ExpectedLightDirectionRounded))
		{
			Result.AddError(FString::Printf(TEXT("Expected (%g, %g, %g) light direction, imported (%g, %g, %g)."),
				ExpectedLightDirectionRounded.X, ExpectedLightDirectionRounded.Y, ExpectedLightDirectionRounded.Z,
				LightDirection.X, LightDirection.Y, LightDirection.Z));
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("LightComponent is null for %s."), *Light->GetName()));
	}

	return Result;
}

FInterchangeTestFunctionResult ULightImportTestFunctions::CheckLightIntensity(ALight* Light, float ExpectedLightIntensity)
{
	FInterchangeTestFunctionResult Result;

	if(ULightComponent* LightComponent = Light->GetLightComponent())
	{
		if(float LightIntensity = LightComponent->Intensity; !FMath::IsNearlyEqual(LightIntensity, ExpectedLightIntensity))
		{
			Result.AddError(FString::Printf(TEXT("Expected %g light intensity, imported %g."), LightIntensity, ExpectedLightIntensity));
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("LightComponent is null for %s."), *Light->GetName()));
	}

	return Result;
}

FInterchangeTestFunctionResult ULightImportTestFunctions::CheckLightColor(ALight* Light, const FLinearColor& ExpectedLightColor)
{
	FInterchangeTestFunctionResult Result;

	if(ULightComponent* LightComponent = Light->GetLightComponent())
	{
		if(LightComponent->LightColor != ExpectedLightColor.ToFColor(true))
		{
			const FLinearColor LightLinearColor = LightComponent->GetLightColor();
			Result.AddError(FString::Printf(TEXT("Expected (%g, %g, %g) light color, imported (%g, %g, %g)."),
											ExpectedLightColor.R, ExpectedLightColor.G, ExpectedLightColor.B,
											LightLinearColor.R, LightLinearColor.G, LightLinearColor.B));
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("LightComponent is null for %s."), *Light->GetName()));
	}

	return Result;
}
