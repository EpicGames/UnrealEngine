// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/MaterialImportTestFunctions.h"

#include "ImportTestFunctions/ImportTestFunctionsBase.h"
#include "Materials/Material.h"
#include "InterchangeTestFunction.h"
#include "Materials/MaterialInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialImportTestFunctions)

namespace UE::Interchange::Tests::Private
{
	TArray<UMaterialInterface*> FilterByMaterialInterfaceSubclass(const TArray<UMaterialInterface*>& MaterialInterfaces, const TSubclassOf<UMaterialInterface>& MaterialInterfaceSubclass)
	{
		TArray<UMaterialInterface*> FilteredMaterialInterfaces;

		for (UMaterialInterface* MaterialInterface : MaterialInterfaces)
		{
			if (MaterialInterface->IsA(MaterialInterfaceSubclass))
			{
				FilteredMaterialInterfaces.Add(MaterialInterface);
			}
		}

		return FilteredMaterialInterfaces;
	}
}

UClass* UMaterialImportTestFunctions::GetAssociatedAssetType() const
{
	return UMaterialInterface::StaticClass();
}

FInterchangeTestFunctionResult UMaterialImportTestFunctions::CheckImportedMaterialCount(const TArray<UMaterialInterface*>& MaterialInterfaces, int32 ExpectedNumberOfImportedMaterials)
{
	using namespace UE::Interchange::Tests::Private;

	FInterchangeTestFunctionResult Result;
	const int32 NumImportedMaterials = FilterByMaterialInterfaceSubclass(MaterialInterfaces, UMaterial::StaticClass()).Num();

	if (NumImportedMaterials != ExpectedNumberOfImportedMaterials)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d materials, imported %d."), ExpectedNumberOfImportedMaterials, NumImportedMaterials));
	}

	return Result;
}

FInterchangeTestFunctionResult UMaterialImportTestFunctions::CheckImportedMaterialInstanceCount(const TArray<UMaterialInterface*>& MaterialInterfaces, int32 ExpectedNumberOfImportedMaterialInstances)
{
	using namespace UE::Interchange::Tests::Private;

	FInterchangeTestFunctionResult Result;
	const int32 NumImportedMaterialInstances = FilterByMaterialInterfaceSubclass(MaterialInterfaces, UMaterialInstance::StaticClass()).Num();

	if (NumImportedMaterialInstances != ExpectedNumberOfImportedMaterialInstances)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d material instances, imported %d."), ExpectedNumberOfImportedMaterialInstances, NumImportedMaterialInstances));
	}

	return Result;
}

FInterchangeTestFunctionResult UMaterialImportTestFunctions::CheckShadingModel(const UMaterialInterface* MaterialInterface, EMaterialShadingModel ExpectedShadingModel)
{
	FInterchangeTestFunctionResult Result;
	const FMaterialShadingModelField ImportedShadingModels = MaterialInterface->GetShadingModels();

	const int32 NumImportedShadingModels = ImportedShadingModels.CountShadingModels();
	if (NumImportedShadingModels != 1)
	{
		Result.AddError(FString::Printf(TEXT("Expected only 1 shading model, imported %d."), NumImportedShadingModels));
	}
	else
	{
		const EMaterialShadingModel ImportedShadingModel = ImportedShadingModels.GetFirstShadingModel();
		if (ImportedShadingModel != ExpectedShadingModel)
		{
			const FString ImportedDisplayValue = UEnum::GetDisplayValueAsText(ImportedShadingModel).ToString();
			const FString ExpectedDisplayValue = UEnum::GetDisplayValueAsText(ExpectedShadingModel).ToString();
			Result.AddError(FString::Printf(TEXT("Expected shading model %s, imported %s."), *ExpectedDisplayValue, *ImportedDisplayValue));
		}
	}

	return Result;
}

FInterchangeTestFunctionResult UMaterialImportTestFunctions::CheckBlendMode(const UMaterialInterface* MaterialInterface, EBlendMode ExpectedBlendMode)
{
	FInterchangeTestFunctionResult Result;
	const EBlendMode ImportedBlendMode = MaterialInterface->GetBlendMode();

	if (ImportedBlendMode != ExpectedBlendMode)
	{
		const FString ImportedDisplayValue = UEnum::GetDisplayValueAsText(ImportedBlendMode).ToString();
		const FString ExpectedDisplayValue = UEnum::GetDisplayValueAsText(ExpectedBlendMode).ToString();
		Result.AddError(FString::Printf(TEXT("Expected blend mode %s, imported %s."), *ExpectedDisplayValue, *ImportedDisplayValue));
	}

	return Result;
}

FInterchangeTestFunctionResult UMaterialImportTestFunctions::CheckIsTwoSided(const UMaterialInterface* MaterialInterface, bool ExpectedIsTwoSided)
{
	FInterchangeTestFunctionResult Result;
	const bool ImportedIsTwoSided = MaterialInterface->IsTwoSided();

	if (ImportedIsTwoSided != ExpectedIsTwoSided)
	{
		const TCHAR* ImportedDisplayValue = ImportedIsTwoSided ? TEXT("True") : TEXT("False");
		const TCHAR* ExpectedDisplayValue = ExpectedIsTwoSided ? TEXT("True") : TEXT("False");
		Result.AddError(FString::Printf(TEXT("Expected two-sided setting to be %s, imported %s."), ExpectedDisplayValue, ImportedDisplayValue));
	}

	return Result;
}

FInterchangeTestFunctionResult UMaterialImportTestFunctions::CheckOpacityMaskClipValue(const UMaterialInterface* MaterialInterface, float ExpectedOpacityMaskClipValue)
{
	FInterchangeTestFunctionResult Result;
	const float ImportedOpacityMaskClipValue = MaterialInterface->GetOpacityMaskClipValue();

	if (!FMath::IsNearlyEqual(ImportedOpacityMaskClipValue, ExpectedOpacityMaskClipValue, UE_KINDA_SMALL_NUMBER))
	{
		Result.AddError(FString::Printf(TEXT("Expected opacity mask clip value %.2f, imported %.2f."), ExpectedOpacityMaskClipValue, ImportedOpacityMaskClipValue));
	}

	return Result;
}

FInterchangeTestFunctionResult UMaterialImportTestFunctions::CheckScalarParameter(const UMaterialInterface* MaterialInterface, const FString& ParameterName, float ExpectedParameterValue)
{
	FInterchangeTestFunctionResult Result;
	float ImportedParameterValue;

	if (!MaterialInterface->GetScalarParameterValue(*ParameterName, ImportedParameterValue))
	{
		Result.AddError(FString::Printf(TEXT("The imported material doesn't contain scalar parameter '%s'."), *ParameterName));
	}
	else if (!FMath::IsNearlyEqual(ImportedParameterValue, ExpectedParameterValue, UE_KINDA_SMALL_NUMBER))
	{
		Result.AddError(FString::Printf(TEXT("For scalar parameter '%s', expected value %f, imported %f."), *ParameterName, ExpectedParameterValue, ImportedParameterValue));
	}

	return Result;
}

FInterchangeTestFunctionResult UMaterialImportTestFunctions::CheckVectorParameter(const UMaterialInterface* MaterialInterface, const FString& ParameterName, FLinearColor ExpectedParameterValue)
{
	FInterchangeTestFunctionResult Result;
	FLinearColor ImportedParameterValue;

	if (!MaterialInterface->GetVectorParameterValue(*ParameterName, ImportedParameterValue))
	{
		Result.AddError(FString::Printf(TEXT("The imported material doesn't contain vector parameter '%s'."), *ParameterName));
	}
	else if (!ImportedParameterValue.Equals(ExpectedParameterValue, UE_KINDA_SMALL_NUMBER))
	{
		Result.AddError(FString::Printf(TEXT("For scalar parameter '%s', expected value %s, imported %s."), *ParameterName, *ExpectedParameterValue.ToString(), *ImportedParameterValue.ToString()));
	}

	return Result;
}
