// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/MaterialXTestFunctions.h"

#include "InterchangeTestFunction.h"

#include "Materials/Material.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialXTestFunctions)

namespace UE::Interchange::Tests::Private
{
	void GetMaterialFunctionConnectedInputs(const UMaterial& Material, TArray<FString>& InputNames)
	{
		bool bHasBaseColor = Material.HasBaseColorConnected();
		bool bHasSubstrateFrontMaterial = Material.HasSubstrateFrontMaterialConnected();

		// For MaterialX those 2 are mutually exclusive, but for backwards compatibility let's keep it that way
		if (!ensure(bHasBaseColor || bHasSubstrateFrontMaterial))
		{
			return;
		}

#if WITH_EDITORONLY_DATA
		const UMaterialEditorOnlyData* MaterialEditorOnly = Material.GetEditorOnlyData();
		UMaterialExpression* Expression = bHasSubstrateFrontMaterial ? MaterialEditorOnly->FrontMaterial.Expression : MaterialEditorOnly->BaseColor.Expression;
		if (UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(Expression))
		{
			InputNames.Empty();

			for (FExpressionInputIterator It{ FunctionCall }; It; ++It)
			{
				if (It->IsConnected())
				{
					InputNames.Add(It->InputName.ToString());
				}
			}
		}
#endif
	}
}

UClass* UMaterialXTestFunctions::GetAssociatedAssetType() const
{
	return UMaterialInterface::StaticClass();
}

FInterchangeTestFunctionResult UMaterialXTestFunctions::CheckConnectedInputCount(const UMaterialInterface* MaterialInterface, int32 ExpectedNumber)
{
	using namespace UE::Interchange::Tests::Private;

	FInterchangeTestFunctionResult Result;

	if (const UMaterial* Material = Cast<UMaterial>(MaterialInterface))
	{
		TArray<FString> InputNames;
		GetMaterialFunctionConnectedInputs(*Material, InputNames);
		if (ExpectedNumber != InputNames.Num())
		{
			Result.AddError(FString::Printf(TEXT("Expected %d connected inputs, found %d connected."), ExpectedNumber, InputNames.Num()));
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("Expected a %s got a %s"), *UMaterial::StaticClass()->GetName(), *MaterialInterface->GetClass()->GetName()));
	}

	return Result;
}

FInterchangeTestFunctionResult UMaterialXTestFunctions::CheckInputConnected(const UMaterialInterface* MaterialInterface, const FString& InputName, bool bIsConnected)
{
	using namespace UE::Interchange::Tests::Private;

	FInterchangeTestFunctionResult Result;

	if (const UMaterial* Material = Cast<UMaterial>(MaterialInterface))
	{
		TArray<FString> InputNames;
		GetMaterialFunctionConnectedInputs(*Material, InputNames);

		if (InputNames.Contains(InputName) != bIsConnected)
		{
			if (bIsConnected)
			{
				Result.AddError(FString::Printf(TEXT("Expected input %s to be connected, found not connected."), *InputName));
			}
			else
			{
				Result.AddError(FString::Printf(TEXT("Expected input %s not to be connected, found connected."), *InputName));
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("Expected a %s got a %s"), *UMaterial::StaticClass()->GetName(), *MaterialInterface->GetClass()->GetName()));
	}

	return Result;
}
