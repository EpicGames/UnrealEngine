// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/PointLightImportTestFunctions.h"
#include "Engine/PointLight.h"
#include "Components/PointLightComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PointLightImportTestFunctions)

UClass* UPointLightImportTestFunctions::GetAssociatedAssetType() const
{
	return APointLight::StaticClass();
}

FInterchangeTestFunctionResult UPointLightImportTestFunctions::CheckLightFalloffExponent(APointLight* Light, float ExpectedLightFalloff)
{
	FInterchangeTestFunctionResult Result;

	if(Light->PointLightComponent)
	{
		if(!FMath::IsNearlyEqual(Light->PointLightComponent->LightFalloffExponent, ExpectedLightFalloff))
		{
			Result.AddError(FString::Printf(TEXT("Expected %g light falloff exponent, imported %g."), Light->PointLightComponent->LightFalloffExponent, ExpectedLightFalloff));
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("LightComponent is null for %s."),	*Light->GetName()));
	}

	return Result;
}
