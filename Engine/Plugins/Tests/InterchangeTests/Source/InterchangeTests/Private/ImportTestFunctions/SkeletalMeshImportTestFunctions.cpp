// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/SkeletalMeshImportTestFunctions.h"

#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/SkinnedAssetCommon.h"
#include "ImportTestFunctions/ImportTestFunctionsBase.h"
#include "InterchangeTestFunction.h"
#include "InterchangeTestsMathUtilities.h"
#include "Materials/MaterialInterface.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "SkeletalMeshAttributes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshImportTestFunctions)


UClass* USkeletalMeshImportTestFunctions::GetAssociatedAssetType() const
{
	return USkeletalMesh::StaticClass();
}


FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckImportedSkeletalMeshCount(const TArray<USkeletalMesh*>& Meshes, int32 ExpectedNumberOfImportedSkeletalMeshes)
{
	FInterchangeTestFunctionResult Result;
	if (Meshes.Num() != ExpectedNumberOfImportedSkeletalMeshes)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d skeletal meshes, imported %d."), ExpectedNumberOfImportedSkeletalMeshes, Meshes.Num()));
	}

	return Result;
}


FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckRenderVertexCount(USkeletalMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfRenderVertices)
{
	FInterchangeTestFunctionResult Result;

	if (Mesh->GetResourceForRendering())
	{
		int32 ImportedLods = Mesh->GetResourceForRendering()->LODRenderData.Num();
		if (LodIndex < 0 || LodIndex >= ImportedLods)
		{
			Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d (imported %d)."), LodIndex, ImportedLods));
		}
		else
		{
			int32 RealVertexCount = Mesh->GetResourceForRendering()->LODRenderData[LodIndex].GetNumVertices();
			if (RealVertexCount != ExpectedNumberOfRenderVertices)
			{
				Result.AddError(FString::Printf(TEXT("For LOD %d, expected %d vertices, imported %d."), LodIndex, ExpectedNumberOfRenderVertices, RealVertexCount));
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data for this skeletalmesh %s."), *Mesh->GetName()));
	}

	return Result;
}

FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckRenderTriangleCount(USkeletalMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfRenderTriangles)
{
	FInterchangeTestFunctionResult Result;

	if (Mesh->GetResourceForRendering())
	{
		int32 ImportedLods = Mesh->GetResourceForRendering()->LODRenderData.Num();
		if (LodIndex < 0 || LodIndex >= ImportedLods)
		{
			Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d (imported %d)."), LodIndex, ImportedLods));
		}
		else
		{
			int32 TriangleCount = Mesh->GetResourceForRendering()->LODRenderData[LodIndex].GetTotalFaces();
			if (TriangleCount != ExpectedNumberOfRenderTriangles)
			{
				Result.AddError(FString::Printf(TEXT("For LOD %d, expected %d triangles, imported %d."), LodIndex, ExpectedNumberOfRenderTriangles, TriangleCount));
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data for this skeletalmesh %s."), *Mesh->GetName()));
	}

	return Result;
}

FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckLodCount(USkeletalMesh* Mesh, int32 ExpectedNumberOfLods)
{
	FInterchangeTestFunctionResult Result;
	if (Mesh->GetResourceForRendering())
	{
		int32 NumLODs = Mesh->GetResourceForRendering()->LODRenderData.Num();
		if (NumLODs != ExpectedNumberOfLods)
		{
			Result.AddError(FString::Printf(TEXT("Expected %d LODs, imported %d."), ExpectedNumberOfLods, NumLODs));
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data for this skeletalmesh %s."), *Mesh->GetName()));
	}

	return Result;
}


FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckMaterialSlotCount(USkeletalMesh* Mesh, int32 ExpectedNumberOfMaterialSlots)
{
	FInterchangeTestFunctionResult Result;

	int32 NumMaterials = Mesh->GetMaterials().Num();
	if (NumMaterials != ExpectedNumberOfMaterialSlots)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d materials, imported %d."), ExpectedNumberOfMaterialSlots, NumMaterials));
	}

	return Result;
}


FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckSectionCount(USkeletalMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfSections)
{
	FInterchangeTestFunctionResult Result;
	if (Mesh->GetResourceForRendering())
	{
		int32 ImportedLods = Mesh->GetResourceForRendering()->LODRenderData.Num();
		if (LodIndex < 0 || LodIndex >= ImportedLods)
		{
			Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d (imported %d)."), LodIndex, ImportedLods));
		}
		else
		{
			int32 NumSections = Mesh->GetResourceForRendering()->LODRenderData[LodIndex].RenderSections.Num();
			if (NumSections != ExpectedNumberOfSections)
			{
				Result.AddError(FString::Printf(TEXT("For LOD %d, expected %d sections, imported %d."), LodIndex, ExpectedNumberOfSections, NumSections));
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data for this skeletalmesh %s."), *Mesh->GetName()));
	}

	return Result;
}


FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckTriangleCountInSection(USkeletalMesh* Mesh, int32 LodIndex, int32 SectionIndex, int32 ExpectedNumberOfTriangles)
{
	FInterchangeTestFunctionResult Result;
	if (Mesh->GetResourceForRendering())
	{
		int32 ImportedLods = Mesh->GetResourceForRendering()->LODRenderData.Num();
		if (LodIndex < 0 || LodIndex >= ImportedLods)
		{
			Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d (imported %d)."), LodIndex, ImportedLods));
		}
		else
		{
			int32 NumSections = Mesh->GetResourceForRendering()->LODRenderData[LodIndex].RenderSections.Num();
			if (SectionIndex >= NumSections)
			{
				Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain section index %d."), SectionIndex));
			}
			else
			{
				int32 NumberOfTriangles = Mesh->GetResourceForRendering()->LODRenderData[LodIndex].RenderSections[SectionIndex].NumTriangles;
				if (NumberOfTriangles != ExpectedNumberOfTriangles)
				{
					Result.AddError(FString::Printf(TEXT("For LOD %d, section index %d, expected %d triangles, imported %d."), LodIndex, SectionIndex, ExpectedNumberOfTriangles, NumberOfTriangles));
				}
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data for this skeletalmesh %s."), *Mesh->GetName()));
	}

	return Result;
}


FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckUVChannelCount(USkeletalMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfUVChannels)
{
	FInterchangeTestFunctionResult Result;
	if (Mesh->GetResourceForRendering())
	{
		int32 ImportedLods = Mesh->GetResourceForRendering()->LODRenderData.Num();
		if (LodIndex < 0 || LodIndex >= ImportedLods)
		{
			Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d (imported %d)."), LodIndex, ImportedLods));
		}
		else
		{
			int32 NumUVs = Mesh->GetResourceForRendering()->LODRenderData[LodIndex].GetNumTexCoords();
			if (NumUVs != ExpectedNumberOfUVChannels)
			{
				Result.AddError(FString::Printf(TEXT("For LOD %d, expected %d UVs, imported %d."), LodIndex, ExpectedNumberOfUVChannels, NumUVs));
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data for this skeletalmesh %s."), *Mesh->GetName()));
	}

	return Result;
}


FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckSectionMaterialName(USkeletalMesh* Mesh, int32 LodIndex, int32 SectionIndex, const FString& ExpectedMaterialName)
{
	FInterchangeTestFunctionResult Result;
	if (Mesh->GetResourceForRendering())
	{
		int32 ImportedLods = Mesh->GetResourceForRendering()->LODRenderData.Num();
		if (LodIndex < 0 || LodIndex >= ImportedLods)
		{
			Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d (imported %d)."), LodIndex, ImportedLods));
		}
		else
		{
			const TArray<FSkelMeshRenderSection>& Sections = Mesh->GetResourceForRendering()->LODRenderData[LodIndex].RenderSections;
			if (SectionIndex >= Sections.Num())
			{
				Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain section index %d (imported %d)."), SectionIndex, Sections.Num()));
			}
			else
			{
				int32 MaterialIndex = Sections[SectionIndex].MaterialIndex;

				const TArray<FSkeletalMaterial>& SkeletalMaterials = Mesh->GetMaterials();
				if (!SkeletalMaterials.IsValidIndex(MaterialIndex) || SkeletalMaterials[MaterialIndex].MaterialInterface == nullptr)
				{
					Result.AddError(FString::Printf(TEXT("The section references a non-existent material (index %d)."), MaterialIndex));
				}
				else
				{
					FString MaterialName = SkeletalMaterials[MaterialIndex].MaterialInterface->GetName();
					if (MaterialName != ExpectedMaterialName)
					{
						Result.AddError(FString::Printf(TEXT("For LOD %d section %d, expected material name %s, imported %s."), LodIndex, SectionIndex, *ExpectedMaterialName, *MaterialName));
					}
				}
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data for this skeletalmesh %s."), *Mesh->GetName()));
	}

	return Result;
}

FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckSectionImportedMaterialSlotName(USkeletalMesh* Mesh, int32 LodIndex, int32 SectionIndex, const FString& ExpectedImportedMaterialSlotName)
{
	FInterchangeTestFunctionResult Result;
	if (Mesh->GetResourceForRendering())
	{
		int32 ImportedLods = Mesh->GetResourceForRendering()->LODRenderData.Num();
		if (LodIndex < 0 || LodIndex >= ImportedLods)
		{
			Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d (imported %d)."), LodIndex, ImportedLods));
		}
		else
		{
			const TArray<FSkelMeshRenderSection>& Sections = Mesh->GetResourceForRendering()->LODRenderData[LodIndex].RenderSections;
			if (SectionIndex >= Sections.Num())
			{
				Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain section index %d (imported %d)."), SectionIndex, Sections.Num()));
			}
			else
			{
				int32 MaterialIndex = Sections[SectionIndex].MaterialIndex;

				const TArray<FSkeletalMaterial>& SkeletalMaterials = Mesh->GetMaterials();
				if (!SkeletalMaterials.IsValidIndex(MaterialIndex))
				{
					Result.AddError(FString::Printf(TEXT("The section references a non-existent material (index %d)."), MaterialIndex));
				}
				else
				{
					const FString ImportedMaterialSlotName = SkeletalMaterials[MaterialIndex].ImportedMaterialSlotName.ToString();
					if (ImportedMaterialSlotName != ExpectedImportedMaterialSlotName)
					{
						Result.AddError(FString::Printf(TEXT("For LOD %d section %d, expected imported material slot name %s, imported %s."), LodIndex, SectionIndex, *ExpectedImportedMaterialSlotName, *ImportedMaterialSlotName));
					}
				}
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data for this skeletalmesh %s."), *Mesh->GetName()));
	}

	return Result;
}

FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckVertexIndexPosition(USkeletalMesh* Mesh, int32 LodIndex, int32 VertexIndex, const FVector& ExpectedVertexPosition)
{
	FInterchangeTestFunctionResult Result;
	if (Mesh->GetResourceForRendering())
	{
		int32 ImportedLods = Mesh->GetResourceForRendering()->LODRenderData.Num();
		if (LodIndex < 0 || LodIndex >= ImportedLods)
		{
			Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d (imported %d)."), LodIndex, ImportedLods));
		}
		else
		{
			int32 VertexCount = Mesh->GetResourceForRendering()->LODRenderData[LodIndex].StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();
			if (VertexIndex >= VertexCount)
			{
				Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain vertex index %d (imported %d)."), VertexIndex, VertexCount));
			}
			else
			{
				using namespace InterchangeTestsMathUtilities;
				const FVector VertexPosition = RoundVectorToDecimalPlaces(FVector(Mesh->GetResourceForRendering()->LODRenderData[LodIndex].StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex)));
				const FVector ExpectedVertexPositionRounded = RoundVectorToDecimalPlaces(ExpectedVertexPosition);
				if (!VertexPosition.Equals(ExpectedVertexPositionRounded, UE_DOUBLE_KINDA_SMALL_NUMBER))
				{
					Result.AddError(FString::Printf(TEXT("For LOD %d vertex index %d, expected position %s, imported %s."), LodIndex, VertexIndex, *ExpectedVertexPosition.ToString(), *VertexPosition.ToString()));
				}
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data for this skeletalmesh %s."), *Mesh->GetName()));
	}

	return Result;
}

FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckVertexIndexNormal(USkeletalMesh* Mesh, int32 LodIndex, int32 VertexIndex, const FVector& ExpectedVertexNormal)
{
	FInterchangeTestFunctionResult Result;
	if (Mesh->GetResourceForRendering())
	{
		int32 ImportedLods = Mesh->GetResourceForRendering()->LODRenderData.Num();
		if (LodIndex < 0 || LodIndex >= ImportedLods)
		{
			Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d (imported %d)."), LodIndex, ImportedLods));
		}
		else
		{
			int32 VertexCount = Mesh->GetResourceForRendering()->LODRenderData[LodIndex].StaticVertexBuffers.StaticMeshVertexBuffer.GetNumVertices();
			if (VertexIndex >= VertexCount)
			{
				Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain vertex index %d (imported %d)."), VertexIndex, VertexCount));
			}
			else
			{
				using namespace InterchangeTestsMathUtilities;
				const FVector VertexNormal = RoundVectorToDecimalPlaces(FVector(Mesh->GetResourceForRendering()->LODRenderData[LodIndex].StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertexIndex)));
				const FVector ExpectedVertexNormalRounded = RoundVectorToDecimalPlaces(ExpectedVertexNormal);
				if (!VertexNormal.Equals(ExpectedVertexNormalRounded, UE_DOUBLE_KINDA_SMALL_NUMBER))
				{
					Result.AddError(FString::Printf(TEXT("For LOD %d vertex index %d, expected normal %s, imported %s."), LodIndex, VertexIndex, *ExpectedVertexNormal.ToString(), *VertexNormal.ToString()));
				}
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data for this skeletalmesh %s."), *Mesh->GetName()));
	}

	return Result;
}

FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckVertexIndexColor(USkeletalMesh* Mesh, int32 LodIndex, int32 VertexIndex, const FColor& ExpectedVertexColor)
{
	FInterchangeTestFunctionResult Result;

	if (Mesh->GetResourceForRendering())
	{
		int32 ImportedLods = Mesh->GetResourceForRendering()->LODRenderData.Num();
		if (LodIndex < 0 || LodIndex >= ImportedLods)
		{
			Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d (imported %d)."), LodIndex, ImportedLods));
		}
		else
		{
			const int32 VertexCount = Mesh->GetResourceForRendering()->LODRenderData[LodIndex].StaticVertexBuffers.ColorVertexBuffer.GetNumVertices();
			if (VertexIndex >= VertexCount)
			{
				Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain vertex index %d (imported %d)."), VertexIndex, VertexCount));
			}
			else
			{
				const FColor& ImportedVertexColor = Mesh->GetResourceForRendering()->LODRenderData[LodIndex].StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex);
				if (ImportedVertexColor != ExpectedVertexColor)
				{
					Result.AddError(FString::Printf(TEXT("For LOD %d, vertex index %d, expected vertex color %s, imported %s."), LodIndex, VertexIndex, *ExpectedVertexColor.ToString(), *ImportedVertexColor.ToString()));
				}
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data for this skeletalmesh %s."), *Mesh->GetName()));
	}

	return Result;
}


FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckMorphTargetCount(USkeletalMesh* Mesh, int32 ExpectedNumberOfMorphTargets)
{
	FInterchangeTestFunctionResult Result;
	constexpr int32 LODIndex0 = 0;
	FMeshDescription* LOD0MeshDescription = Mesh->GetMeshDescription(LODIndex0);
	if (LOD0MeshDescription == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("could not retrieve LOD 0 for skeletalmesh %s"), *Mesh->GetName()));
	}
	else 
	{
		FSkeletalMeshConstAttributes SkelMeshAttributes(*LOD0MeshDescription);
		if (SkelMeshAttributes.GetMorphTargetNames().Num() != ExpectedNumberOfMorphTargets)
		{
			Result.AddError(FString::Printf(TEXT("found %d Morph Targets for this skeletalmesh %s - but expected %d Morph Targets"), SkelMeshAttributes.GetMorphTargetNames().Num(), *Mesh->GetName(), ExpectedNumberOfMorphTargets));
		}
	}

	return Result;
}

FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckMorphTargetName(USkeletalMesh* Mesh, int32 MorphTargetIndex, const FString& ExpectedMorphTargetName)
{
	FInterchangeTestFunctionResult Result;
	constexpr int32 LODIndex0 = 0;
	FMeshDescription* LOD0MeshDescription = Mesh->GetMeshDescription(LODIndex0);
	if (LOD0MeshDescription == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("could not retrieve LOD 0 for skeletalmesh %s"), *Mesh->GetName()));
	}
	else
	{
		FSkeletalMeshConstAttributes SkelMeshAttributes(*LOD0MeshDescription);
		
		if (MorphTargetIndex < 0 || MorphTargetIndex >= SkelMeshAttributes.GetMorphTargetNames().Num())
		{
			Result.AddError(FString::Printf(TEXT("imported skeletalmesh %s doesn't have Morph Target at index %d"), *Mesh->GetName(), MorphTargetIndex));
		}
		else
		{	
			const FString MorphTargetNameString = SkelMeshAttributes.GetMorphTargetNames()[MorphTargetIndex].ToString();
			if (!MorphTargetNameString.Equals(ExpectedMorphTargetName))
			{
				Result.AddError(FString::Printf(TEXT("morph target at index %d has name %s - but expected %s"), MorphTargetIndex, *MorphTargetNameString, *ExpectedMorphTargetName));
			}
		}
	}

	return Result;
}

FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckBoneCount(USkeletalMesh* Mesh, int32 ExpectedNumberOfBones)
{
	FInterchangeTestFunctionResult Result;

	USkeleton* Skeleton = Mesh->GetSkeleton();
	if (Skeleton == nullptr)
	{
		if (ExpectedNumberOfBones != 0)
		{
			Result.AddError(FString::Printf(TEXT("No skeleton found - but expected %d bones"), ExpectedNumberOfBones));
		}
	}
	else
	{
		int32 NumberOfBones = Skeleton->GetReferenceSkeleton().GetNum();
		if (NumberOfBones != ExpectedNumberOfBones)
		{
			Result.AddError(FString::Printf(TEXT("Expected %d bones, imported %d."), ExpectedNumberOfBones, NumberOfBones));
		}
	}

	return Result;
}


FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckBonePosition(USkeletalMesh* Mesh, int32 BoneIndex, const FVector& ExpectedBonePosition)
{
	FInterchangeTestFunctionResult Result;

	USkeleton* Skeleton = Mesh->GetSkeleton();
	if (Skeleton == nullptr)
	{
		Result.AddError(TEXT("No skeleton found."));
	}
	else
	{
		int32 NumberOfBones = Skeleton->GetReferenceSkeleton().GetNum();
		if (BoneIndex >= NumberOfBones)
		{
			Result.AddError(FString::Printf(TEXT("Expected bone index %d, but only imported %d bones."), BoneIndex, NumberOfBones));
		}
		else
		{
			using namespace InterchangeTestsMathUtilities;
			const FVector BonePosition = RoundVectorToDecimalPlaces(Mesh->GetRefSkeleton().GetRefBonePose()[BoneIndex].GetLocation());
			const FVector ExpectedBonePositionRounded = RoundVectorToDecimalPlaces(ExpectedBonePosition);
			if (!BonePosition.Equals(ExpectedBonePositionRounded, UE_DOUBLE_KINDA_SMALL_NUMBER))
			{
				Result.AddError(FString::Printf(TEXT("For bone index %d, expected position %s, imported %s."), BoneIndex, *ExpectedBonePosition.ToString(), *BonePosition.ToString()));
			}
		}
	}

	return Result;
}

FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckSocketCount(USkeletalMesh* Mesh, int32 ExpectedSocketCount)
{
	FInterchangeTestFunctionResult Result;
	if (Mesh->NumSockets() != ExpectedSocketCount)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d sockets on the skeletal mesh, found %d"), ExpectedSocketCount, Mesh->NumSockets()));
	}
	return Result;
}

FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckSocketName(USkeletalMesh* Mesh, int32 SocketIndex, const FString& ExpectedSocketName)
{
	FInterchangeTestFunctionResult Result;
	if (USkeletalMeshSocket* Socket = Mesh->GetSocketByIndex(SocketIndex))
	{
		if (!ExpectedSocketName.Equals(Socket->SocketName.ToString()))
		{
			Result.AddError(FString::Printf(TEXT("Expected Socket with name %s at index %d for skeletal mesh, received %s"), *ExpectedSocketName, SocketIndex, *Socket->SocketName.ToString()));
		}
	}
	else 
	{
		Result.AddError(FString::Printf(TEXT("Invalid SocketIndex(%d) for skeletal mesh"), SocketIndex));
	}
	return Result;
}

FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckSocketLocation(USkeletalMesh* Mesh, int32 SocketIndex, const FVector& ExpectedSocketLocation)
{
	FInterchangeTestFunctionResult Result;
	if (USkeletalMeshSocket* Socket = Mesh->GetSocketByIndex(SocketIndex))
	{
		using namespace InterchangeTestsMathUtilities;
		const FVector RoundedSocketLocation = RoundVectorToDecimalPlaces(Socket->RelativeLocation);
		const FVector RoundedExpectedLocation = RoundVectorToDecimalPlaces(ExpectedSocketLocation);
		if (!RoundedExpectedLocation.Equals(RoundedSocketLocation, UE_DOUBLE_KINDA_SMALL_NUMBER))
		{
			Result.AddError(FString::Printf(TEXT("Expected Socket at index %d to have location %s for skeletal mesh, received %s"), SocketIndex, *RoundedSocketLocation.ToString(), *RoundedExpectedLocation.ToString()));
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("Invalid SocketIndex(%d) for skeletal mesh"), SocketIndex));
	}
	return Result;
}

FInterchangeTestFunctionResult USkeletalMeshImportTestFunctions::CheckSkinnedVertexCountForBone(USkeletalMesh* Mesh, const FString& BoneName, bool bTestFirstAlternateProfile, int32 ExpectedSkinnedVertexCount)
{
	FInterchangeTestFunctionResult Result;

	int32 BoneIndex = Mesh->GetRefSkeleton().FindBoneIndex(*BoneName);
	if (!Mesh->GetRefSkeleton().IsValidIndex(BoneIndex))
	{
		Result.AddError(FString::Printf(TEXT("Could not find bone '%s'."), *BoneName));
	}
	else
	{
		if (Mesh->GetImportedModel() && Mesh->GetImportedModel()->LODModels.IsValidIndex(0))
		{
			int32 SkinnedVerticesForBone = 0;
			auto IncrementInfluence = [&SkinnedVerticesForBone, &BoneIndex](const FSkelMeshSection& Section,
																			const FBoneIndexType(&InfluenceBones)[MAX_TOTAL_INFLUENCES],
																			const uint16(&InfluenceWeights)[MAX_TOTAL_INFLUENCES])
			{
				for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
				{
					if (InfluenceWeights[InfluenceIndex] == 0)
					{
						// Influences are sorted by weight so no need to go further then a zero weight
						break;
					}
					if (Section.BoneMap[InfluenceBones[InfluenceIndex]] == BoneIndex)
					{
						SkinnedVerticesForBone++;
						break;
					}
				}
			};

			if (!bTestFirstAlternateProfile)
			{
				for (const FSkelMeshSection& Section : Mesh->GetImportedModel()->LODModels[0].Sections)
				{
					const int32 SectionVertexCount = Section.SoftVertices.Num();
					// Find the number of vertices skinned to this bone
					for (int32 SectionVertexIndex = 0; SectionVertexIndex < SectionVertexCount; ++SectionVertexIndex)
					{
						const FSoftSkinVertex& Vertex = Section.SoftVertices[SectionVertexIndex];
						IncrementInfluence(Section, Vertex.InfluenceBones, Vertex.InfluenceWeights);
					}
				}
			}
			else
			{
				if (Mesh->GetSkinWeightProfiles().Num() > 0)
				{
					const int32 TotalVertexCount = Mesh->GetImportedModel()->LODModels[0].NumVertices;
					const FSkinWeightProfileInfo& SkinWeightProfile = Mesh->GetSkinWeightProfiles()[0];
					const FImportedSkinWeightProfileData& SkinWeightData = Mesh->GetImportedModel()->LODModels[0].SkinWeightProfiles.FindChecked(SkinWeightProfile.Name);

					if (SkinWeightData.SkinWeights.Num() != TotalVertexCount)
					{
						Result.AddError(TEXT("Unable to find alternate skinning profile, please uncheck the 'test alternate profile' box."));
					}
					else
					{
						int32 TotalVertexIndex = 0;
						for (const FSkelMeshSection& Section : Mesh->GetImportedModel()->LODModels[0].Sections)
						{
							const int32 SectionVertexCount = Section.SoftVertices.Num();
							//Find the number of vertex skin by this bone
							for (int32 SectionVertexIndex = 0; SectionVertexIndex < SectionVertexCount; ++SectionVertexIndex, ++TotalVertexIndex)
							{
								const FRawSkinWeight& SkinWeight = SkinWeightData.SkinWeights[TotalVertexIndex];
								IncrementInfluence(Section, SkinWeight.InfluenceBones, SkinWeight.InfluenceWeights);
							}
						}
					}
				}
			}

			if (SkinnedVerticesForBone != ExpectedSkinnedVertexCount)
			{
				Result.AddError(FString::Printf(TEXT("For bone '%s', expected %d vertices, imported %d."), *BoneName, ExpectedSkinnedVertexCount, SkinnedVerticesForBone));
			}
		}
		else
		{
			Result.AddError(TEXT("No valid mesh geometry found to find the vertex count"));
		}
	}

	return Result;
}


