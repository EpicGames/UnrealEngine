// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/SkeletonImportTestFunctions.h"
#include "Animation/Skeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletonImportTestFunctions)


UClass* USkeletonImportTestFunctions::GetAssociatedAssetType() const
{
	return USkeleton::StaticClass();
}


FInterchangeTestFunctionResult USkeletonImportTestFunctions::CheckImportedSkeletonCount(const TArray<USkeleton*>& Skeletons, int32 ExpectedNumberOfImportedSkeletons)
{
	FInterchangeTestFunctionResult Result;
	if (Skeletons.Num() != ExpectedNumberOfImportedSkeletons)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d skeleton, imported %d."), ExpectedNumberOfImportedSkeletons, Skeletons.Num()));
	}

	return Result;
}