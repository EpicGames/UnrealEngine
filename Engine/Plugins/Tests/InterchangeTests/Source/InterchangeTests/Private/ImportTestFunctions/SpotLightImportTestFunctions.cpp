// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/SpotLightImportTestFunctions.h"
#include "Engine/SpotLight.h"
#include "Components/SpotLightComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpotLightImportTestFunctions)

UClass* USpotLightImportTestFunctions::GetAssociatedAssetType() const
{
	return ASpotLight::StaticClass();
}

FInterchangeTestFunctionResult USpotLightImportTestFunctions::CheckLightInnerConeAngle(ASpotLight* Light, float ExpectedLightInnerConeAngle)
{
	FInterchangeTestFunctionResult Result;

	if(Light->SpotLightComponent)
	{
		if(!FMath::IsNearlyEqual(Light->SpotLightComponent->InnerConeAngle, ExpectedLightInnerConeAngle))
		{
			Result.AddError(FString::Printf(TEXT("Expected %g light inner cone angle, imported %g."), Light->SpotLightComponent->InnerConeAngle, ExpectedLightInnerConeAngle));
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("LightComponent is null for %s."), *Light->GetName()));
	}

	return Result;
}

FInterchangeTestFunctionResult USpotLightImportTestFunctions::CheckLightOuterConeAngle(ASpotLight* Light, float ExpectedLightOuterConeAngle)
{
	FInterchangeTestFunctionResult Result;

	if(Light->SpotLightComponent && !FMath::IsNearlyEqual(Light->SpotLightComponent->OuterConeAngle, ExpectedLightOuterConeAngle))
	{
		Result.AddError(FString::Printf(TEXT("Expected %g light outer cone angle, imported %g."), Light->SpotLightComponent->OuterConeAngle, ExpectedLightOuterConeAngle));
	}

	return Result;
}
