// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/StaticMeshImportTestFunctions.h"

#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshSocket.h"
#include "Engine/StaticMeshSourceData.h"
#include "ImportTestFunctions/ImportTestFunctionsBase.h"
#include "InterchangeTestFunction.h"
#include "InterchangeTestsMathUtilities.h"
#include "Materials/MaterialInterface.h"
#include "PhysicsEngine/BodySetup.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticMeshImportTestFunctions)

namespace UE::Interchange::Tests::Private
{
	void ReportPropertyDeltaAsErrors(FInterchangeTestFunctionResult& Result, TPropertyValueIterator<const FProperty>& PropertyValueIteratorA, TPropertyValueIterator<const FProperty>& PropertyValueIteratorB)
	{
		for (; PropertyValueIteratorA && PropertyValueIteratorB; ++PropertyValueIteratorA, ++PropertyValueIteratorB)
		{
			const FProperty* PropertyA = PropertyValueIteratorA->Key;
			const FProperty* PropertyB = PropertyValueIteratorB->Key;

			if (!PropertyA || !PropertyB)
			{
				Result.AddError(TEXT("Unexpected error (invalid FProperty)"));
				return;
			}

			if (!PropertyA->SameType(PropertyB))
			{
				Result.AddError(FString::Printf(TEXT("Unexpected error (FProperty %s::%s doesn't match FProperty %s::%s)"),
					*PropertyA->GetOwnerStruct()->GetName(), *PropertyA->GetName(),
					*PropertyB->GetOwnerStruct()->GetName(), *PropertyB->GetName()));
			}

			const void* ValueA = PropertyValueIteratorA->Value;
			const void* ValueB = PropertyValueIteratorB->Value;

			if (!PropertyA->Identical(ValueA, ValueB))
			{
				FString ValueAString;
				PropertyA->ExportTextItem_Direct(ValueAString, ValueA, nullptr, nullptr, PPF_None);

				FString ValueBString;
				PropertyB->ExportTextItem_Direct(ValueBString, ValueB, nullptr, nullptr, PPF_None);

				Result.AddError(FString::Printf(TEXT("Expected property (%s::%s=%s), imported (%s::%s=%s)"),
					*PropertyA->GetOwnerStruct()->GetName(), *PropertyA->GetName(), *ValueAString,
					*PropertyB->GetOwnerStruct()->GetName(), *PropertyB->GetName(), *ValueBString));
			}
		}
	}
}


UClass* UStaticMeshImportTestFunctions::GetAssociatedAssetType() const
{
	return UStaticMesh::StaticClass();
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckImportedStaticMeshCount(const TArray<UStaticMesh*>& Meshes, int32 ExpectedNumberOfImportedStaticMeshes)
{
	FInterchangeTestFunctionResult Result;
	if (Meshes.Num() != ExpectedNumberOfImportedStaticMeshes)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d static meshes, imported %d."), ExpectedNumberOfImportedStaticMeshes, Meshes.Num()));
	}

	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckVertexCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfVertices)
{
	FInterchangeTestFunctionResult Result;

	const FMeshDescription* MeshDescription = Mesh->GetMeshDescription(LodIndex);
	if (MeshDescription == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else
	{
		int32 RealVertexCount = MeshDescription->Vertices().Num();
		if (RealVertexCount != ExpectedNumberOfVertices)
		{
			Result.AddError(FString::Printf(TEXT("For LOD %d, expected %d vertices, imported %d."), LodIndex, ExpectedNumberOfVertices, RealVertexCount));
		}
	}

	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckRenderVertexCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfRenderVertices)
{
	FInterchangeTestFunctionResult Result;

	if (LodIndex < 0 || LodIndex >= Mesh->GetNumLODs())
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else
	{
		int32 RealVertexCount = Mesh->GetNumVertices(LodIndex);
		if (RealVertexCount != ExpectedNumberOfRenderVertices)
		{
			Result.AddError(FString::Printf(TEXT("For LOD %d, expected %d vertices, imported %d."), LodIndex, ExpectedNumberOfRenderVertices, RealVertexCount));
		}
	}

	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckLodCount(UStaticMesh* Mesh, int32 ExpectedNumberOfLods)
{
	FInterchangeTestFunctionResult Result;

	int32 NumLODs = Mesh->GetNumSourceModels();
	if (NumLODs != ExpectedNumberOfLods)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d LODs, imported %d."), ExpectedNumberOfLods, NumLODs));
	}

	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckMaterialSlotCount(UStaticMesh* Mesh, int32 ExpectedNumberOfMaterialSlots)
{
	FInterchangeTestFunctionResult Result;

	int32 NumMaterials = Mesh->GetStaticMaterials().Num();
	if (NumMaterials != ExpectedNumberOfMaterialSlots)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d materials, imported %d."), ExpectedNumberOfMaterialSlots, NumMaterials));
	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckImportedMaterialSlotName(UStaticMesh* Mesh, int32 MaterialIndex, const FString& ExpectedImportedMaterialSlotName)
{
	FInterchangeTestFunctionResult Result;

	if (!Mesh->GetStaticMaterials().IsValidIndex(MaterialIndex))
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't have a material at index %d, mesh material count is %d."), MaterialIndex, Mesh->GetStaticMaterials().Num()));
	}
	else
	{
		const FString ImportedMaterialSlotNameAtIndex = Mesh->GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName.ToString();
		if (!ImportedMaterialSlotNameAtIndex.Equals(ExpectedImportedMaterialSlotName, ESearchCase::CaseSensitive))
		{
			Result.AddError(FString::Printf(TEXT("For material index %d, expected imported material slot name [%s], imported [%s]."), MaterialIndex, *ExpectedImportedMaterialSlotName, *ImportedMaterialSlotNameAtIndex));
		}
	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckPolygonGroupCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfPolygonGroups)
{
	FInterchangeTestFunctionResult Result;

	const FMeshDescription* MeshDescription = Mesh->GetMeshDescription(LodIndex);
	if (MeshDescription == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else
	{
		int32 RealNumberOfPolygonGroups = MeshDescription->PolygonGroups().Num();
		if (RealNumberOfPolygonGroups != ExpectedNumberOfPolygonGroups)
		{
			Result.AddError(FString::Printf(TEXT("For LOD %d, expected %d polygon groups, imported %d."), LodIndex, ExpectedNumberOfPolygonGroups, RealNumberOfPolygonGroups));
		}
	}

	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckSectionCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfSections)
{
	FInterchangeTestFunctionResult Result;

	if (LodIndex < 0 || LodIndex >= Mesh->GetNumLODs())
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else
	{
		int32 NumSections = Mesh->GetNumSections(LodIndex);
		if (NumSections != ExpectedNumberOfSections)
		{
			Result.AddError(FString::Printf(TEXT("For LOD %d, expected %d sections, imported %d."), LodIndex, ExpectedNumberOfSections, NumSections));
		}

	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckRenderVertexIndexNormal(UStaticMesh* Mesh, int32 LodIndex, int32 VertexIndex, const FVector4f& ExpectedVertexNormal)
{
	FInterchangeTestFunctionResult Result;

	if (LodIndex < 0 || LodIndex >= Mesh->GetNumLODs())
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else if (Mesh->GetRenderData() && Mesh->GetRenderData()->LODResources.IsValidIndex(LodIndex))
	{
		const int32 NumVertices = Mesh->GetRenderData()->LODResources[LodIndex].GetNumVertices();
		if (VertexIndex >= NumVertices)
		{
			Result.AddError(FString::Printf(TEXT("The imported LOD render data doesn't contain vertex index %d."), VertexIndex));
		}
		else
		{
			using namespace InterchangeTestsMathUtilities;
			const FVector4f VertexNormal = RoundVector4ToDecimalPlaces(Mesh->GetRenderData()->LODResources[LodIndex].VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertexIndex));
			const FVector4f ExpectedVertexNormalRounded = RoundVector4ToDecimalPlaces(ExpectedVertexNormal);
			if (!VertexNormal.Equals(ExpectedVertexNormalRounded, UE_KINDA_SMALL_NUMBER))
			{
				Result.AddError(FString::Printf(TEXT("For LOD %d, vertex index %d, expected vertex normals %s, imported %s."), LodIndex, VertexIndex, *VertexNormal.ToString(), *ExpectedVertexNormal.ToString()));
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data For LOD %d."), LodIndex));
	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckRenderVertexIndexColor(UStaticMesh* Mesh, int32 LodIndex, int32 VertexIndex, const FColor& ExpectedVertexColor)
{
	FInterchangeTestFunctionResult Result;

	if (LodIndex < 0 || LodIndex >= Mesh->GetNumLODs())
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else if (Mesh->GetRenderData() && Mesh->GetRenderData()->LODResources.IsValidIndex(LodIndex))
	{
		const int32 NumVertices = Mesh->GetRenderData()->LODResources[LodIndex].VertexBuffers.ColorVertexBuffer.GetNumVertices();
		if (VertexIndex >= NumVertices)
		{
			Result.AddError(FString::Printf(TEXT("The imported LOD render color buffer doesn't contain vertex index %d."), VertexIndex));
		}
		else
		{
			const FColor& ImportedVertexColor = Mesh->GetRenderData()->LODResources[LodIndex].VertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex);
			if (ImportedVertexColor != ExpectedVertexColor)
			{
				Result.AddError(FString::Printf(TEXT("For LOD %d, vertex index %d, expected vertex color %s, imported %s."), LodIndex, VertexIndex, *ExpectedVertexColor.ToString(), *ImportedVertexColor.ToString()));
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data For LOD %d."), LodIndex));
	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckRenderHasVertexColors(UStaticMesh* Mesh, int32 LodIndex, bool bExpectedHasVertexColors)
{
	FInterchangeTestFunctionResult Result;

	if (LodIndex < 0 || LodIndex >= Mesh->GetNumLODs())
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else if (Mesh->GetRenderData() && Mesh->GetRenderData()->LODResources.IsValidIndex(LodIndex))
	{
		const bool bImportedHasVertexColors = Mesh->GetRenderData()->LODResources[LodIndex].VertexBuffers.ColorVertexBuffer.GetNumVertices() != 0;
		if (bImportedHasVertexColors != bExpectedHasVertexColors)
		{
			const TCHAR* ImportedDisplayValue = bImportedHasVertexColors ? TEXT("True") : TEXT("False");
			const TCHAR* ExpectedDisplayValue = bExpectedHasVertexColors ? TEXT("True") : TEXT("False");
			Result.AddError(FString::Printf(TEXT("For LOD %d, expected vertex colors %s, imported %s."), LodIndex, ExpectedDisplayValue, ImportedDisplayValue));
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data For LOD %d."), LodIndex));
	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckTriangleCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedTotalNumberOfTriangles)
{
	FInterchangeTestFunctionResult Result;

	const FMeshDescription* MeshDescription = Mesh->GetMeshDescription(LodIndex);
	if (MeshDescription == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else
	{
		int32 RealNumberOfTriangles = MeshDescription->Triangles().Num();
		if (RealNumberOfTriangles != ExpectedTotalNumberOfTriangles)
		{
			Result.AddError(FString::Printf(TEXT("For LOD %d, expected %d triangles, imported %d."), LodIndex, ExpectedTotalNumberOfTriangles, RealNumberOfTriangles));
		}
	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckRenderTriangleCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedTotalNumberOfTriangles)
{
	FInterchangeTestFunctionResult Result;

	if (LodIndex < 0 || LodIndex >= Mesh->GetNumLODs())
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else if (Mesh->GetRenderData() && Mesh->GetRenderData()->LODResources.IsValidIndex(LodIndex))
	{
		const int32 NumTriangles = Mesh->GetRenderData()->LODResources[LodIndex].GetNumTriangles();
		if (NumTriangles != ExpectedTotalNumberOfTriangles)
		{
			Result.AddError(FString::Printf(TEXT("For LOD %d, expected %d triangles, imported %d."), LodIndex, ExpectedTotalNumberOfTriangles, NumTriangles));
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data For LOD %d."), LodIndex));
	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckPolygonCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedTotalNumberOfPolygons)
{
	FInterchangeTestFunctionResult Result;

	const FMeshDescription* MeshDescription = Mesh->GetMeshDescription(LodIndex);
	if (MeshDescription == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else
	{
		int32 RealNumberOfPolygons = MeshDescription->Polygons().Num();
		if (RealNumberOfPolygons != ExpectedTotalNumberOfPolygons)
		{
			Result.AddError(FString::Printf(TEXT("For LOD %d, expected %d polygons, imported %d."), LodIndex, ExpectedTotalNumberOfPolygons, RealNumberOfPolygons));
		}
	}

	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckThatMeshHasQuadsOrNgons(UStaticMesh* Mesh, int32 LodIndex, bool bMeshHasQuadsOrNgons)
{
	FInterchangeTestFunctionResult Result;

	const FMeshDescription* MeshDescription = Mesh->GetMeshDescription(LodIndex);
	if (MeshDescription == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else
	{
		int32 NumberOfTriangles = MeshDescription->Triangles().Num();
		int32 NumberOfPolygons = MeshDescription->Polygons().Num();
		bool bReallyHasQuads = (NumberOfTriangles > NumberOfPolygons);

		if (bReallyHasQuads != bMeshHasQuadsOrNgons)
		{
			Result.AddError(FString::Printf(TEXT("For LOD %d, expected has quads %d, imported %d."), LodIndex, bMeshHasQuadsOrNgons, bReallyHasQuads));
		}
	}

	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckTriangleCountInPolygonGroup(UStaticMesh* Mesh, int32 LodIndex, int32 PolygonGroupIndex, int32 ExpectedNumberOfTriangles)
{
	FInterchangeTestFunctionResult Result;

	const FMeshDescription* MeshDescription = Mesh->GetMeshDescription(LodIndex);
	if (MeshDescription == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else
	{
		int32 RealNumberOfTriangles = MeshDescription->GetNumPolygonGroupTriangles(PolygonGroupIndex);
		if (RealNumberOfTriangles != ExpectedNumberOfTriangles)
		{
			Result.AddError(FString::Printf(TEXT("For LOD %d PolygonGroup %d, expected %d triangles, imported %d."), LodIndex, PolygonGroupIndex, ExpectedNumberOfTriangles, RealNumberOfTriangles));
		}
	}

	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckPolygonCountInPolygonGroup(UStaticMesh* Mesh, int32 LodIndex, int32 PolygonGroupIndex, int32 ExpectedNumberOfPolygons)
{
	FInterchangeTestFunctionResult Result;

	const FMeshDescription* MeshDescription = Mesh->GetMeshDescription(LodIndex);
	if (MeshDescription == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else
	{
		int32 RealNumberOfPolygons = MeshDescription->GetNumPolygonGroupPolygons(PolygonGroupIndex);
		if (RealNumberOfPolygons != ExpectedNumberOfPolygons)
		{
			Result.AddError(FString::Printf(TEXT("For LOD %d PolygonGroup %d, expected %d polygons, imported %d."), LodIndex, PolygonGroupIndex, ExpectedNumberOfPolygons, RealNumberOfPolygons));
		}
	}

	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckUVChannelCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfUVChannels)
{
	FInterchangeTestFunctionResult Result;

	const FMeshDescription* MeshDescription = Mesh->GetMeshDescription(LodIndex);
	if (MeshDescription == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else
	{
		FStaticMeshConstAttributes Attributes(*MeshDescription);

		int32 NumberOfUVChannels = Attributes.GetVertexInstanceUVs().GetNumChannels();
		if (NumberOfUVChannels != ExpectedNumberOfUVChannels)
		{
			Result.AddError(FString::Printf(TEXT("Expected %d UV Channels, imported %d."), ExpectedNumberOfUVChannels, NumberOfUVChannels));
		}
	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckRenderUVChannelCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfUVChannels)
{
	FInterchangeTestFunctionResult Result;

	if (LodIndex < 0 || LodIndex >= Mesh->GetNumLODs())
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else if (Mesh->GetRenderData() && Mesh->GetRenderData()->LODResources.IsValidIndex(LodIndex))
	{
		const int32 NumberOfUVChannels = Mesh->GetRenderData()->LODResources[LodIndex].GetNumTexCoords();
		if (NumberOfUVChannels != ExpectedNumberOfUVChannels)
		{
			Result.AddError(FString::Printf(TEXT("Expected %d UV Channels, imported %d."), ExpectedNumberOfUVChannels, NumberOfUVChannels));
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data For LOD %d."), LodIndex));
	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckPolygonGroupImportedMaterialSlotName(UStaticMesh* Mesh, int32 LodIndex, int32 PolygonGroupIndex, const FString& ExpectedImportedMaterialSlotName)
{
	FInterchangeTestFunctionResult Result;

	const FMeshDescription* MeshDescription = Mesh->GetMeshDescription(LodIndex);
	if (MeshDescription == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else
	{
		FStaticMeshConstAttributes Attributes(*MeshDescription);

		FString MaterialSlotName = Attributes.GetPolygonGroupMaterialSlotNames()[PolygonGroupIndex].ToString();
		if (MaterialSlotName != ExpectedImportedMaterialSlotName)
		{
			Result.AddError(FString::Printf(TEXT("For LOD %d PolygonGroup %d, expected slot name %s, imported %s."), LodIndex, PolygonGroupIndex, *ExpectedImportedMaterialSlotName, *MaterialSlotName));
		}
	}

	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckSectionMaterialIndex(UStaticMesh* Mesh, int32 LodIndex, int32 SectionIndex, int32 ExpectedMaterialIndex)
{
	FInterchangeTestFunctionResult Result;

	if (LodIndex < 0 || LodIndex >= Mesh->GetNumLODs())
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else if (Mesh->GetRenderData() && Mesh->GetRenderData()->LODResources.IsValidIndex(LodIndex))
	{
		const FStaticMeshSectionArray& Sections = Mesh->GetRenderData()->LODResources[LodIndex].Sections;
		if (SectionIndex >= Sections.Num())
		{
			Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain section index %d."), SectionIndex));
		}
		else
		{
			int32 MaterialIndex = Sections[SectionIndex].MaterialIndex;

			if (MaterialIndex != ExpectedMaterialIndex)
			{
				Result.AddError(FString::Printf(TEXT("For LOD %d section %d, expected material index %d, imported %d."), LodIndex, SectionIndex, ExpectedMaterialIndex, MaterialIndex));
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data For LOD %d."), LodIndex));
	}

	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckSectionMaterialName(UStaticMesh* Mesh, int32 LodIndex, int32 SectionIndex, const FString& ExpectedMaterialName)
{
	FInterchangeTestFunctionResult Result;

	if (LodIndex < 0 || LodIndex >= Mesh->GetNumLODs())
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else if (Mesh->GetRenderData() && Mesh->GetRenderData()->LODResources.IsValidIndex(LodIndex))
	{
		const FStaticMeshSectionArray& Sections = Mesh->GetRenderData()->LODResources[LodIndex].Sections;
		if (SectionIndex >= Sections.Num())
		{
			Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain section index %d."), SectionIndex));
		}
		else
		{
			int32 MaterialIndex = Sections[SectionIndex].MaterialIndex;

			const TArray<FStaticMaterial>& StaticMaterials = Mesh->GetStaticMaterials();
			if (!StaticMaterials.IsValidIndex(MaterialIndex) || StaticMaterials[MaterialIndex].MaterialInterface == nullptr)
			{
				Result.AddError(FString::Printf(TEXT("The section references a non-existent material (index %d)."), MaterialIndex));
			}
			else
			{
				FString MaterialName = StaticMaterials[MaterialIndex].MaterialInterface->GetName();
				if (MaterialName != ExpectedMaterialName)
				{
					Result.AddError(FString::Printf(TEXT("For LOD %d section %d, expected material name %s, imported %s."), LodIndex, SectionIndex, *ExpectedMaterialName, *MaterialName));
				}
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data For LOD %d."), LodIndex));
	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckSectionImportedMaterialSlotName(UStaticMesh* Mesh, int32 LodIndex, int32 SectionIndex, const FString& ExpectedImportedMaterialSlotName)
{
	FInterchangeTestFunctionResult Result;

	if (LodIndex < 0 || LodIndex >= Mesh->GetNumLODs())
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else if (Mesh->GetRenderData() && Mesh->GetRenderData()->LODResources.IsValidIndex(LodIndex))
	{
		const FStaticMeshSectionArray& Sections = Mesh->GetRenderData()->LODResources[LodIndex].Sections;
		if (SectionIndex >= Sections.Num())
		{
			Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain section index %d."), SectionIndex));
		}
		else
		{
			int32 MaterialIndex = Sections[SectionIndex].MaterialIndex;

			const TArray<FStaticMaterial>& StaticMaterials = Mesh->GetStaticMaterials();
			if (!StaticMaterials.IsValidIndex(MaterialIndex))
			{
				Result.AddError(FString::Printf(TEXT("The section references a non-existent material (index %d)."), MaterialIndex));
			}
			else
			{
				FString ImportedMaterialSlotName = StaticMaterials[MaterialIndex].ImportedMaterialSlotName.ToString();
				if (ImportedMaterialSlotName != ExpectedImportedMaterialSlotName)
				{
					Result.AddError(FString::Printf(TEXT("For LOD %d section %d, expected imported material slot name %s, imported %s."), LodIndex, SectionIndex, *ExpectedImportedMaterialSlotName, *ImportedMaterialSlotName));
				}
			}
		}
	}
	else
	{
		Result.AddError(FString::Printf(TEXT("No valid render data For LOD %d."), LodIndex));
	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckVertexIndexPosition(UStaticMesh* Mesh, int32 LodIndex, int32 VertexIndex, const FVector& ExpectedVertexPosition)
{
	FInterchangeTestFunctionResult Result;

	const FMeshDescription* MeshDescription = Mesh->GetMeshDescription(LodIndex);
	if (MeshDescription == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain LOD index %d."), LodIndex));
	}
	else
	{
		FStaticMeshConstAttributes Attributes(*MeshDescription);
		
		if (VertexIndex >= Attributes.GetVertexPositions().GetNumElements())
		{
			Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain a vertex of index %d."), VertexIndex));
		}
		else
		{
			using namespace InterchangeTestsMathUtilities;
			const FVector VertexPosition = RoundVectorToDecimalPlaces(FVector(Attributes.GetVertexPositions()[VertexIndex]));
			const FVector ExpectedVertexPositionRounded = RoundVectorToDecimalPlaces(ExpectedVertexPosition);
			if (!VertexPosition.Equals(ExpectedVertexPositionRounded, UE_DOUBLE_KINDA_SMALL_NUMBER))
			{				
				Result.AddError(FString::Printf(TEXT("For LOD %d vertex index %d, expected position %s, imported %s."), LodIndex, VertexIndex, *ExpectedVertexPosition.ToString(), *VertexPosition.ToString()));
			}
		}
	}

	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckSimpleCollisionPrimitiveCount(UStaticMesh* Mesh, int32 ExpectedSphereElementCount, int32 ExpectedBoxElementCount, int32 ExpectedCapsuleElementCount, int32 ExpectedConvexElementCount, int32 ExpectedTaperedCapsuleElementCount)
{
	FInterchangeTestFunctionResult Result;

	UBodySetup* BodySetup = Mesh->GetBodySetup();
	if (BodySetup == nullptr)
	{
		Result.AddError(TEXT("Didn't find a BodySetup on the mesh"));
	}
	else
	{
		// If there was no collision imported, collision will be automatically generated as a kdop-18 convex primitive.
		// We should not count this when checking imported simple collision, hence we treat the convex element count as zero if there was no customized collision.
		bool bImportedCollision = Mesh->GetCustomizedCollision();

		int32 SphereElementCount = BodySetup->AggGeom.SphereElems.Num();
		int32 BoxElementCount = BodySetup->AggGeom.BoxElems.Num();
		int32 CapsuleElementCount = BodySetup->AggGeom.SphylElems.Num();
		int32 ConvexElementCount = bImportedCollision ? BodySetup->AggGeom.ConvexElems.Num() : 0;
		int32 TaperedCapsuleElementCount = BodySetup->AggGeom.TaperedCapsuleElems.Num();

		if (SphereElementCount != ExpectedSphereElementCount)
		{
			Result.AddError(FString::Printf(TEXT("Expected %d sphere elements, imported %d."), ExpectedSphereElementCount, SphereElementCount));
		}

		if (BoxElementCount != ExpectedBoxElementCount)
		{
			Result.AddError(FString::Printf(TEXT("Expected %d box elements, imported %d."), ExpectedBoxElementCount, BoxElementCount));
		}

		if (CapsuleElementCount != ExpectedCapsuleElementCount)
		{
			Result.AddError(FString::Printf(TEXT("Expected %d capsule elements, imported %d."), ExpectedCapsuleElementCount, CapsuleElementCount));
		}

		if (ConvexElementCount != ExpectedConvexElementCount)
		{
			Result.AddError(FString::Printf(TEXT("Expected %d convex elements, imported %d."), ExpectedConvexElementCount, ConvexElementCount));
		}

		if (TaperedCapsuleElementCount != ExpectedTaperedCapsuleElementCount)
		{
			Result.AddError(FString::Printf(TEXT("Expected %d tapered capsule elements, imported %d."), ExpectedTaperedCapsuleElementCount, TaperedCapsuleElementCount));
		}
	}


	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckSocketCount(UStaticMesh* Mesh, int32 ExpectedSocketCount)
{
	FInterchangeTestFunctionResult Result;

	int32 SocketCount = Mesh->Sockets.Num();
	if (SocketCount != ExpectedSocketCount)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d sockets, imported %d."), ExpectedSocketCount, SocketCount));
	}

	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckSocketName(UStaticMesh* Mesh, int32 SocketIndex, const FString& ExpectedSocketName)
{
	FInterchangeTestFunctionResult Result;

	int32 SocketCount = Mesh->Sockets.Num();
	if (SocketIndex >= SocketCount)
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain %d sockets."), SocketIndex));
	}

	FString SocketName = Mesh->Sockets[SocketIndex]->SocketName.ToString();
	if (SocketName != ExpectedSocketName)
	{
		Result.AddError(FString::Printf(TEXT("Expected socket name '%s', imported '%s'"), *ExpectedSocketName, *SocketName));
	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckSocketLocation(UStaticMesh* Mesh, int32 SocketIndex, const FVector& ExpectedSocketLocation)
{
	FInterchangeTestFunctionResult Result;

	if (Mesh->Sockets.IsValidIndex(SocketIndex))
	{
		using namespace InterchangeTestsMathUtilities;

		const FVector SocketLocation = RoundVectorToDecimalPlaces(Mesh->Sockets[SocketIndex]->RelativeLocation);
		const FVector ExpectedSocketLocationRounded = RoundVectorToDecimalPlaces(ExpectedSocketLocation);
		if (!SocketLocation.Equals(ExpectedSocketLocationRounded, UE_DOUBLE_KINDA_SMALL_NUMBER))
		{
			Result.AddError(FString::Printf(TEXT("Expected socket location %s, imported %s"), *ExpectedSocketLocation.ToString(), *SocketLocation.ToString()));
		}
	}
	else 
	{
		Result.AddError(FString::Printf(TEXT("The imported mesh doesn't contain socket with index %d."), SocketIndex));
	}

	return Result;
}


FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckAgainstGroundTruth(UStaticMesh* Mesh, TSoftObjectPtr<UStaticMesh> MeshToCompare,
	bool bCheckVertexCountEqual,
	bool bCheckTriangleCountEqual,
	bool bCheckUVChannelCountEqual,
	bool bCheckCollisionPrimitiveCountEqual,
	bool bCheckVertexPositionsEqual,
	bool bCheckNormalsEqual
)
{
	FInterchangeTestFunctionResult Result;

	UStaticMesh* GroundTruth = MeshToCompare.LoadSynchronous();
	if (GroundTruth == nullptr)
	{
		Result.AddError(FString::Printf(TEXT("Unable to find ground truth mesh '%s'"), *MeshToCompare.ToString()));
		return Result;
	}

	int32 NumLods = Mesh->GetNumSourceModels();
	int32 ExpectedNumLods = GroundTruth->GetNumSourceModels();

	if (NumLods != ExpectedNumLods)
	{
		Result.AddError(FString::Printf(TEXT("Number of imported LODs don't match: expected %d, imported %d"), ExpectedNumLods, NumLods));
		return Result;
	}

	if (bCheckVertexCountEqual)
	{
		for (int32 LodIndex = 0; LodIndex < NumLods; LodIndex++)
		{
			const FMeshDescription* MD = Mesh->GetMeshDescription(LodIndex);
			const FMeshDescription* GroundTruthMD = GroundTruth->GetMeshDescription(LodIndex);

			int32 VertexCount = MD->Vertices().Num();
			int32 ExpectedVertexCount = GroundTruthMD->Vertices().Num();

			if (VertexCount != ExpectedVertexCount)
			{
				Result.AddError(FString::Printf(TEXT("Number of vertices don't match in LOD %d: expected %d, imported %d"), LodIndex, ExpectedVertexCount, VertexCount));
			}
		}
	}

	if (bCheckTriangleCountEqual)
	{
		for (int32 LodIndex = 0; LodIndex < NumLods; LodIndex++)
		{
			const FMeshDescription* MD = Mesh->GetMeshDescription(LodIndex);
			const FMeshDescription* GroundTruthMD = GroundTruth->GetMeshDescription(LodIndex);

			int32 TriangleCount = MD->Triangles().Num();
			int32 ExpectedTriangleCount = GroundTruthMD->Triangles().Num();

			if (TriangleCount != ExpectedTriangleCount)
			{
				Result.AddError(FString::Printf(TEXT("Number of triangles don't match in LOD %d: expected %d, imported %d"), LodIndex, ExpectedTriangleCount, TriangleCount));
			}
		}
	}

	if (bCheckUVChannelCountEqual)
	{
		for (int32 LodIndex = 0; LodIndex < NumLods; LodIndex++)
		{
			const FMeshDescription* MD = Mesh->GetMeshDescription(LodIndex);
			const FMeshDescription* GroundTruthMD = GroundTruth->GetMeshDescription(LodIndex);

			FStaticMeshConstAttributes Attributes(*MD);
			FStaticMeshConstAttributes GroundTruthAttributes(*GroundTruthMD);

			int32 UVChannels = Attributes.GetVertexInstanceUVs().GetNumChannels();
			int32 ExpectedUVChannels = GroundTruthAttributes.GetVertexInstanceUVs().GetNumChannels();

			if (UVChannels != ExpectedUVChannels)
			{
				Result.AddError(FString::Printf(TEXT("Number of UV channels don't match in LOD %d: expected %d, imported %d"), LodIndex, ExpectedUVChannels, UVChannels));
			}
		}
	}

	if (bCheckCollisionPrimitiveCountEqual)
	{
		UBodySetup* BodySetup = Mesh->GetBodySetup();
		UBodySetup* GroundTruthBodySetup = GroundTruth->GetBodySetup();

		if (BodySetup == nullptr || GroundTruthBodySetup == nullptr)
		{
			Result.AddError(TEXT("No collision data found to compare"));
		}
		else
		{
			const FKAggregateGeom& AggGeom = BodySetup->AggGeom;
			const FKAggregateGeom& GroundTruthAggGeom = GroundTruthBodySetup->AggGeom;

			if (AggGeom.BoxElems.Num() != GroundTruthAggGeom.BoxElems.Num())
			{
				Result.AddError(FString::Printf(TEXT("Box collision count mismatch: expected %d, imported %d"), GroundTruthAggGeom.BoxElems.Num(), AggGeom.BoxElems.Num()));
			}

			if (AggGeom.SphereElems.Num() != GroundTruthAggGeom.SphereElems.Num())
			{
				Result.AddError(FString::Printf(TEXT("Sphere collision count mismatch: expected %d, imported %d"), GroundTruthAggGeom.SphereElems.Num(), AggGeom.SphereElems.Num()));
			}

			if (AggGeom.ConvexElems.Num() != GroundTruthAggGeom.ConvexElems.Num())
			{
				Result.AddError(FString::Printf(TEXT("Convex collision count mismatch: expected %d, imported %d"), GroundTruthAggGeom.ConvexElems.Num(), AggGeom.ConvexElems.Num()));
			}

			if (AggGeom.SphylElems.Num() != GroundTruthAggGeom.SphylElems.Num())
			{
				Result.AddError(FString::Printf(TEXT("Capsule collision count mismatch: expected %d, imported %d"), GroundTruthAggGeom.SphylElems.Num(), AggGeom.SphylElems.Num()));
			}

			if (AggGeom.TaperedCapsuleElems.Num() != GroundTruthAggGeom.TaperedCapsuleElems.Num())
			{
				Result.AddError(FString::Printf(TEXT("Tapered capsule collision count mismatch: expected %d, imported %d"), GroundTruthAggGeom.TaperedCapsuleElems.Num(), AggGeom.TaperedCapsuleElems.Num()));
			}
		}
	}

	if (bCheckVertexPositionsEqual)
	{
		for (int32 LodIndex = 0; LodIndex < NumLods; LodIndex++)
		{
			const FMeshDescription* MD = Mesh->GetMeshDescription(LodIndex);
			const FMeshDescription* GroundTruthMD = GroundTruth->GetMeshDescription(LodIndex);

			FStaticMeshConstAttributes Attributes(*MD);
			TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();

			FStaticMeshConstAttributes GroundTruthAttributes(*GroundTruthMD);
			TVertexAttributesConstRef<FVector3f> GroundTruthVertexPositions = GroundTruthAttributes.GetVertexPositions();

			int32 VertexCount = Attributes.GetVertexPositions().GetNumElements();
			int32 ExpectedVertexCount = GroundTruthAttributes.GetVertexPositions().GetNumElements();

			if (VertexCount != ExpectedVertexCount)
			{
				Result.AddError(FString::Printf(TEXT("Number of vertices don't match in LOD %d: expected %d, imported %d"), LodIndex, ExpectedVertexCount, VertexCount));
			}
			else
			{
				for (int32 VertexIndex = 0; VertexIndex < VertexCount; VertexIndex++)
				{
					using namespace InterchangeTestsMathUtilities;
					const FVector3f V0 = RoundVectorToDecimalPlaces(VertexPositions[VertexIndex]);
					const FVector3f V1 = RoundVectorToDecimalPlaces(GroundTruthVertexPositions[VertexIndex]);

					if (!V0.Equals(V1))
					{
						Result.AddError(FString::Printf(TEXT("LOD %d: vertex index %d: position mismatch: expected %s, imported %s"), LodIndex, VertexIndex, *V1.ToString(), *V0.ToString()));
					}
				}
			}
		}
	}

	if (bCheckNormalsEqual)
	{
		for (int32 LodIndex = 0; LodIndex < NumLods; LodIndex++)
		{
			const FMeshDescription* MD = Mesh->GetMeshDescription(LodIndex);
			const FMeshDescription* GroundTruthMD = GroundTruth->GetMeshDescription(LodIndex);

			FStaticMeshConstAttributes Attributes(*MD);
			TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();

			FStaticMeshConstAttributes GroundTruthAttributes(*GroundTruthMD);
			TVertexInstanceAttributesConstRef<FVector3f> GroundTruthVertexInstanceNormals = GroundTruthAttributes.GetVertexInstanceNormals();

			int32 NormalsCount = Attributes.GetVertexInstanceNormals().GetNumElements();
			int32 ExpectedNormalsCount = GroundTruthAttributes.GetVertexInstanceNormals().GetNumElements();

			if (NormalsCount != ExpectedNormalsCount)
			{
				Result.AddError(FString::Printf(TEXT("Number of vertices don't match in LOD %d: expected %d, imported %d"), LodIndex, ExpectedNormalsCount, NormalsCount));
			}
			else
			{
				for (int32 NormalIndex = 0; NormalIndex < NormalsCount; NormalIndex++)
				{
					using namespace InterchangeTestsMathUtilities;
					const FVector3f N0 = RoundVectorToDecimalPlaces(VertexInstanceNormals[NormalIndex]);
					const FVector3f N1 = RoundVectorToDecimalPlaces(GroundTruthVertexInstanceNormals[NormalIndex]);

					if (!N0.Equals(N1))
					{
						Result.AddError(FString::Printf(TEXT("LOD %d: normal index %d mismatch: expected %s, imported %s"), LodIndex, NormalIndex, *N1.ToString(), *N0.ToString()));
					}
				}
			}
		}
	}

	// Don't keep the ground truth asset lying around.
	// @todo: this doesn't work for some reason; find out why.
//	GroundTruth->ClearFlags(RF_Standalone | RF_WasLoaded | RF_LoadCompleted);

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckBuildSettings(UStaticMesh* Mesh, int32 LodIndex, const FMeshBuildSettings& ExpectedBuildSettings)
{
	using namespace UE::Interchange::Tests::Private;

	FInterchangeTestFunctionResult Result;

	if (Mesh->GetSourceModel(LodIndex).BuildSettings != ExpectedBuildSettings)
	{
		const UStruct* MeshBuildSettingsStruct = FMeshBuildSettings::StaticStruct();
		TPropertyValueIterator<const FProperty> MeshPropertyValueIterator(MeshBuildSettingsStruct, &Mesh->GetSourceModel(LodIndex).BuildSettings);
		TPropertyValueIterator<const FProperty> ExpectedPropertyValueIterator(MeshBuildSettingsStruct, &ExpectedBuildSettings);

		ReportPropertyDeltaAsErrors(Result, MeshPropertyValueIterator, ExpectedPropertyValueIterator);

	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckNaniteSettings(UStaticMesh* Mesh, const FMeshNaniteSettings& ExpectedNaniteSettings)
{
	using namespace UE::Interchange::Tests::Private;

	FInterchangeTestFunctionResult Result;

	if (Mesh->GetNaniteSettings() != ExpectedNaniteSettings)
	{
		const UStruct* NaniteSettingsStruct = FMeshNaniteSettings::StaticStruct();
		TPropertyValueIterator<const FProperty> MeshPropertyValueIterator(NaniteSettingsStruct, &Mesh->GetNaniteSettings());
		TPropertyValueIterator<const FProperty> ExpectedPropertyValueIterator(NaniteSettingsStruct, &ExpectedNaniteSettings);

		ReportPropertyDeltaAsErrors(Result, MeshPropertyValueIterator, ExpectedPropertyValueIterator);
	}

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckBoundingBoxSizeGreaterThan(UStaticMesh* Mesh, const FVector& GreaterThanSize)
{
	using namespace InterchangeTestsMathUtilities;
	FBox BBox = Mesh->GetBoundingBox();
	const FVector Size = RoundVectorToDecimalPlaces(BBox.GetSize());
	const FVector GreaterThanSizeRounded = RoundVectorToDecimalPlaces(GreaterThanSize);
	
	FInterchangeTestFunctionResult Result;

	auto CompareDimension = [](FInterchangeTestFunctionResult& Result, double AxisSize, double ExpectedAxisSize, FString AxisName)
	{
		if (AxisSize <= ExpectedAxisSize)
		{
			Result.AddError(FString::Printf(TEXT("BoundingBoxSize.%s[%lf] is not greater than expected[%lf]."), *AxisName, AxisSize, ExpectedAxisSize));
		}
	};
	
	CompareDimension(Result, Size.X, GreaterThanSizeRounded.X, "X");
	CompareDimension(Result, Size.Y, GreaterThanSizeRounded.Y, "Y");
	CompareDimension(Result, Size.Z, GreaterThanSizeRounded.Z, "Z");

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckBoundingBoxSizeLessThan(UStaticMesh* Mesh, const FVector& LessThanSize)
{
	using namespace InterchangeTestsMathUtilities;
	FBox BBox = Mesh->GetBoundingBox();
	const FVector Size = RoundVectorToDecimalPlaces(BBox.GetSize());
	const FVector LessThanSizeRounded = RoundVectorToDecimalPlaces(LessThanSize);

	FInterchangeTestFunctionResult Result;

	auto CompareDimension = [](FInterchangeTestFunctionResult& Result, double AxisSize, double ExpectedAxisSize, FString AxisName)
	{
		if (AxisSize >= ExpectedAxisSize)
		{
			Result.AddError(FString::Printf(TEXT("BoundingBoxSize.%s[%lf] is not less than expected[%lf]."), *AxisName, AxisSize, ExpectedAxisSize));
		}
	};

	CompareDimension(Result, Size.X, LessThanSizeRounded.X, "X");
	CompareDimension(Result, Size.Y, LessThanSizeRounded.Y, "Y");
	CompareDimension(Result, Size.Z, LessThanSizeRounded.Z, "Z");

	return Result;
}

FInterchangeTestFunctionResult UStaticMeshImportTestFunctions::CheckBoundingBoxSize(UStaticMesh* Mesh, const FVector& ExpectedSize)
{
	using namespace InterchangeTestsMathUtilities;
	FBox BBox = Mesh->GetBoundingBox();
	const FVector Size = RoundVectorToDecimalPlaces(BBox.GetSize());
	const FVector ExpectedSizeRounded = RoundVectorToDecimalPlaces(ExpectedSize);

	FInterchangeTestFunctionResult Result;

	auto CompareDimension = [](FInterchangeTestFunctionResult& Result, double AxisSize, double ExpectedAxisSize, FString AxisName)
	{
		if (!FMath::IsNearlyEqual(AxisSize, ExpectedAxisSize))
		{
			Result.AddError(FString::Printf(TEXT("BoundingBoxSize.%s[%lf] is not equal to expected[%lf]."), *AxisName, AxisSize, ExpectedAxisSize));
		}
	};

	CompareDimension(Result, Size.X, ExpectedSizeRounded.X, "X");
	CompareDimension(Result, Size.Y, ExpectedSizeRounded.Y, "Y");
	CompareDimension(Result, Size.Z, ExpectedSizeRounded.Z, "Z");

	return Result;
}