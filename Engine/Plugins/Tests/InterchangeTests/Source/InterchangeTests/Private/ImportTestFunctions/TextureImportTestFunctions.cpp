// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportTestFunctions/TextureImportTestFunctions.h"
#include "Engine/Texture.h"
#include "ImportTestFunctions/ImportTestFunctionsBase.h"
#include "InterchangeTestFunction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureImportTestFunctions)


UClass* UTextureImportTestFunctions::GetAssociatedAssetType() const
{
	return UTexture::StaticClass();
}

FInterchangeTestFunctionResult UTextureImportTestFunctions::CheckTextureType(const TArray<UTexture*>& Textures, TSubclassOf<UTexture> ExpectedTextureType, int32 ExpectedNumber)
{
	FInterchangeTestFunctionResult Result;
	int32 Count = 0;
	for (UTexture* Texture : Textures)
	{
		if (Texture->IsA(ExpectedTextureType.Get()))
		{
			Count++;
		}
	}

	if (Count != ExpectedNumber)
	{
		Result.AddError(FString::Printf(TEXT("Expected to have %d textures of type %s, imported %d."), ExpectedNumber, *ExpectedTextureType.Get()->GetName(), Count));
	}

	return Result;
}

FInterchangeTestFunctionResult UTextureImportTestFunctions::CheckImportedTextureCount(const TArray<UTexture*>& Textures, int32 ExpectedNumberOfImportedTextures)
{
	FInterchangeTestFunctionResult Result;
	if (Textures.Num() != ExpectedNumberOfImportedTextures)
	{
		Result.AddError(FString::Printf(TEXT("Expected %d textures, imported %d."), ExpectedNumberOfImportedTextures, Textures.Num()));
	}

	return Result;
}

FInterchangeTestFunctionResult UTextureImportTestFunctions::CheckTextureFilter(const UTexture* Texture, TextureFilter ExpectedTextureFilter)
{
	FInterchangeTestFunctionResult Result;
	const TextureFilter ImportedTextureFilter = Texture->Filter;

	if (ImportedTextureFilter != ExpectedTextureFilter)
	{
		const FString ImportedDisplayValue = UEnum::GetDisplayValueAsText(ImportedTextureFilter).ToString();
		const FString ExpectedDisplayValue = UEnum::GetDisplayValueAsText(ExpectedTextureFilter).ToString();
		Result.AddError(FString::Printf(TEXT("Expected texture filter mode %s, imported %s."), *ExpectedDisplayValue, *ImportedDisplayValue));
	}

	return Result;
}

FInterchangeTestFunctionResult UTextureImportTestFunctions::CheckTextureAddressX(const UTexture* Texture, TextureAddress ExpectedTextureAddressX)
{
	FInterchangeTestFunctionResult Result;
	const TextureAddress ImportedTextureAddressX = Texture->GetTextureAddressX();

	if (ImportedTextureAddressX != ExpectedTextureAddressX)
	{
		const FString ImportedDisplayValue = UEnum::GetDisplayValueAsText(ImportedTextureAddressX).ToString();
		const FString ExpectedDisplayValue = UEnum::GetDisplayValueAsText(ExpectedTextureAddressX).ToString();
		Result.AddError(FString::Printf(TEXT("For X-axis, expected texture address mode %s, imported %s."), *ExpectedDisplayValue, *ImportedDisplayValue));
	}

	return Result;
}

FInterchangeTestFunctionResult UTextureImportTestFunctions::CheckTextureAddressY(const UTexture* Texture, TextureAddress ExpectedTextureAddressY)
{
	FInterchangeTestFunctionResult Result;
	const TextureAddress ImportedTextureAddressY = Texture->GetTextureAddressY();

	if (ImportedTextureAddressY != ExpectedTextureAddressY)
	{
		const FString ImportedDisplayValue = UEnum::GetDisplayValueAsText(ImportedTextureAddressY).ToString();
		const FString ExpectedDisplayValue = UEnum::GetDisplayValueAsText(ExpectedTextureAddressY).ToString();
		Result.AddError(FString::Printf(TEXT("For Y-axis, expected texture address mode %s, imported %s."), *ExpectedDisplayValue, *ImportedDisplayValue));
	}

	return Result;
}

FInterchangeTestFunctionResult UTextureImportTestFunctions::CheckTextureAddressZ(const UTexture* Texture, TextureAddress ExpectedTextureAddressZ)
{
	FInterchangeTestFunctionResult Result;
	const TextureAddress ImportedTextureAddressZ = Texture->GetTextureAddressZ();

	if (ImportedTextureAddressZ != ExpectedTextureAddressZ)
	{
		const FString ImportedDisplayValue = UEnum::GetDisplayValueAsText(ImportedTextureAddressZ).ToString();
		const FString ExpectedDisplayValue = UEnum::GetDisplayValueAsText(ExpectedTextureAddressZ).ToString();
		Result.AddError(FString::Printf(TEXT("For Z-axis, expected texture address mode %s, imported %s."), *ExpectedDisplayValue, *ImportedDisplayValue));
	}

	return Result;
}
