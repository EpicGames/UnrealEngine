// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeAutomatedTestUtils.h"

#include "AssetCompilingManager.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AutomationBlueprintFunctionLibrary.h"
#include "Camera/CameraActor.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "EngineUtils.h"
#include "HAL/FileManager.h"
#include "InterchangeHelper.h"
#include "InterchangeImportTestData.h"
#include "InterchangeImportTestPlan.h"
#include "InterchangeImportTestStepBase.h"
#include "InterchangeImportTestStepImport.h"
#include "InterchangeImportTestStepReimport.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "Subsystems/UnrealEditorSubsystem.h"

namespace InterchangeAutomationTestUtils::Private
{
	TTuple<UE::Interchange::FAssetImportResultPtr, UE::Interchange::FSceneImportResultPtr> StartCurrentStep(FInterchangeAutomationTestStepData& TestStepData)
	{
		if (TestStepData.bIsReimportStep)
		{
			if (ensure(TestStepData.TestPlanData.TestPlan->ReimportStack.IsValidIndex(TestStepData.StepIndex)))
			{
				return TestStepData.TestPlanData.TestPlan->ReimportStack[TestStepData.StepIndex]->StartStep(TestStepData.TestPlanData);
			}
			
			return { nullptr, nullptr };
		}
		else
		{
			return TestStepData.TestPlanData.TestPlan->ImportStep->StartStep(TestStepData.TestPlanData);
		}
	}

	void FinishCurrentStep(FInterchangeAutomationTestStepData& TestStepData, FAutomationTestBase* CurrentTest)
	{
		if (TestStepData.bIsReimportStep)
		{
			TestStepData.TestPlanData.TestPlan->ReimportStack[TestStepData.StepIndex]->FinishStep(TestStepData.TestPlanData, CurrentTest);
		}
		else
		{
			TestStepData.TestPlanData.TestPlan->ImportStep->FinishStep(TestStepData.TestPlanData, CurrentTest);
		}
	}

	FString GetContextString(const FInterchangeAutomationTestStepData& TestStepData)
	{
		if (TestStepData.bIsReimportStep)
		{
			return TestStepData.TestPlanData.AssetName.ToString() + TEXT(": ") + TestStepData.TestPlanData.TestPlan->ReimportStack[TestStepData.StepIndex]->GetContextString();
		}
		else
		{
			return TestStepData.TestPlanData.AssetName.ToString() + TEXT(": ") + TestStepData.TestPlanData.TestPlan->ImportStep->GetContextString();
		}
	}

	FString GetScreenshotNameString(const FInterchangeAutomationTestStepData& TestStepData)
	{
		if (TestStepData.bIsReimportStep)
		{
			return FString::Printf(TEXT("%s_ReimportStep_%d"), *TestStepData.TestPlanData.AssetName.ToString(), TestStepData.StepIndex);
		}
		else
		{
			return FString::Printf(TEXT("%s_ImportStep"), *TestStepData.TestPlanData.AssetName.ToString());
		}
	}
}


bool FInterchangeIntializeStepCommand::Update()
{
	using namespace InterchangeAutomationTestUtils::Private;

	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);
	TestStepData->bIsReimportStep = bIsReimport;
	TestStepData->StepIndex = StepIndex;
	TestStepData->TestPlanData.ImportedAssets.Empty();
	TestStepData->Results = StartCurrentStep(*TestStepData);

	if (UE::Interchange::FAssetImportResultPtr AssetImportResults = TestStepData->Results.Get<0>())
	{
		AssetImportResults->WaitUntilDone();
	}

	if (UE::Interchange::FSceneImportResultPtr SceneImportResults = TestStepData->Results.Get<1>())
	{
		SceneImportResults->WaitUntilDone();
	}

	return true;
}

bool FInterchangeInterStepCollectResultCommand::Update()
{
	using namespace InterchangeAutomationTestUtils::Private;
	if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
	{
		const FString ContextString = GetContextString(*TestStepData);

		CurrentTest->PushContext(ContextString);

		// Fill out list of result objects in the data object.
		// These are the UObject* results corresponding to imported assets.
		if (UE::Interchange::FAssetImportResultPtr AssetImportResults = TestStepData->Results.Get<0>())
		{
			for (UObject* ImportedObject : AssetImportResults->GetImportedObjects())
			{
				TestStepData->TestPlanData.ResultObjects.AddUnique(ImportedObject);
			}

			// Also add the InterchangeResultsContainer to the data so that tests can be run on it
			// (e.g. to check whether something imported with a specific expected error)
			TestStepData->TestPlanData.InterchangeResults = AssetImportResults->GetResults();

			// Fill out list of imported assets as FAssetData
			for (UObject* Object : AssetImportResults->GetImportedObjects())
			{
				TestStepData->TestPlanData.ImportedAssets.Emplace(Object);
			}
		}

		if (UE::Interchange::FSceneImportResultPtr SceneImportResults = TestStepData->Results.Get<1>())
		{
			for (UObject* ImportedObject : SceneImportResults->GetImportedObjects())
			{
				TestStepData->TestPlanData.ResultObjects.AddUnique(ImportedObject);
			}

			if (TestStepData->TestPlanData.InterchangeResults)
			{
				TestStepData->TestPlanData.InterchangeResults->Append(SceneImportResults->GetResults());
			}
			else
			{
				TestStepData->TestPlanData.InterchangeResults = SceneImportResults->GetResults();
			}
		}

		// We don't need the Interchange results object any more, as we've already taken everything from it that we need.
		// If we don't reset it, it will hold on to the trashed versions of the imported assets during GC.
		TestStepData->Results = {};
		//Make sure interchange result are not garbage collected in case we reload some packages
		if (TestStepData->TestPlanData.InterchangeResults)
		{
			TestStepData->TestPlanData.InterchangeResults->SetFlags(RF_Standalone);
		}

		{
			for (int32 ResultObjectIndex = 0; ResultObjectIndex < TestStepData->TestPlanData.ResultObjects.Num(); ++ResultObjectIndex)
			{
				UObject* ResultObject = TestStepData->TestPlanData.ResultObjects[ResultObjectIndex];
				if (UWorld* World = Cast<UWorld>(ResultObject))
				{
					if (TestStepData->TestPlanData.TestPlan->GetCurrentWorld() == World)
					{
						continue;
					}
					else
					{
						ResultObject->SetFlags(RF_Standalone);
					}
				}
			}
		}
	}
	return true;
}

bool FInterchangeSetupScreenshotViewportCommand::Update()
{
	if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
	{
		TestStepData->RequiredScreenshotCount++;
		TArray<AActor*> ActorsInLevel;

		if (ScreenshotParameters.bAutoFocus)
		{
			for (UObject* Object : TestStepData->TestPlanData.ResultObjects)
			{
				if (!Object || (Object && !Object->IsA<AActor>()))
				{
					continue;
				}

				if (ScreenshotParameters.FocusActorName.IsEmpty())
				{
					// No specific actor is focused, focus on the whole scene.
					ActorsInLevel.Add(Cast<AActor>(Object));
					continue;
				}

				if (ScreenshotParameters.FocusActorName.Equals(Object->GetName()))
				{
					if (ScreenshotParameters.FocusActorClass.Get())
					{
						if (Object->GetClass()->IsChildOf(ScreenshotParameters.FocusActorClass.Get()))
						{
							ActorsInLevel.Add(Cast<AActor>(Object));
						}
						else
						{
							CurrentTest->AddWarning(FString::Printf(TEXT("Actor with name '%s' found but it doesn't have the actor class: %s"), *ScreenshotParameters.FocusActorName, *ScreenshotParameters.FocusActorClass.Get()->GetName()));
						}
					}
					else
					{
						if (!ActorsInLevel.IsEmpty())
						{
							CurrentTest->AddWarning(FString::Printf(TEXT("Current Scene contains more than one actors with the name %s. If you would like to focus on a specific actor, please consider providing the actor class in the screenshot parameters."), *Object->GetName()));
						}

						ActorsInLevel.Add(Cast<AActor>(Object));
					}
				}
			}
		}

		if (!ScreenshotParameters.bAutoFocus || (ScreenshotParameters.bAutoFocus && !ActorsInLevel.IsEmpty()))
		{
			TestStepData->bCanTakeScreenshot = true;

			// Cache current view mode and wireframe opacity to be restored later during cleanup.
			TestStepData->CachedScreenshotParameters.WireframeOpacity = UAutomationBlueprintFunctionLibrary::GetEditorActiveViewportWireframeOpacity();
			TestStepData->CachedScreenshotParameters.ViewMode = UAutomationBlueprintFunctionLibrary::GetEditorActiveViewportViewMode();

			UAutomationBlueprintFunctionLibrary::SetEditorActiveViewportViewMode(ScreenshotParameters.ViewMode);
			UAutomationBlueprintFunctionLibrary::SetEditorActiveViewportWireframeOpacity(ScreenshotParameters.WireframeOpacity);

			// Always use camera transform if not using auto focus.
			if (!ScreenshotParameters.bAutoFocus)
			{
				if (UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>())
				{
					UnrealEditorSubsystem->SetLevelViewportCameraInfo(ScreenshotParameters.CameraLocation, ScreenshotParameters.CameraRotation);
				}
			}
			else
			{
				// Reset Viewport Camera Position to Origin for consistent results.
				if (UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>())
				{
					UnrealEditorSubsystem->SetLevelViewportCameraInfo(FVector::ZeroVector, FRotator::ZeroRotator);
				}

				constexpr bool bActiveViewportOnly = true;
				GEditor->MoveViewportCamerasToActor(ActorsInLevel, bActiveViewportOnly);
			}
		}
		else
		{
			TestStepData->bCanTakeScreenshot = false;
			CurrentTest->AddError(TEXT("Current Scene could not focus on required actors. Screenshot will not be captured."));
		}
	}
	return true;
}

bool FInterchangeCaptureScreenshotCommand::Update()
{
	if (!TestStepData->bCanTakeScreenshot)
	{
		return true;
	}

	if (TestStepData->ScreenshotTask == nullptr)
	{
		using namespace InterchangeAutomationTestUtils::Private;
		
		constexpr bool bMaskEnabled = false;
		constexpr bool bCaptureHDR = false;
		constexpr float Delay = 0.0f;

		TestStepData->ScreenshotTask = UAutomationBlueprintFunctionLibrary::TakeHighResScreenshot(
			1280, 720, 
			GetScreenshotNameString(*TestStepData), 
			nullptr, 
			bMaskEnabled, bCaptureHDR, 
			ScreenshotParameters.ComparisonTolerance,
			TEXT(""),
			Delay,
			ScreenshotParameters.bForceGameView);
	}

	if (TestStepData->ScreenshotTask && !TestStepData->ScreenshotTask->IsValidTask())
	{
		if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
		{
			CurrentTest->AddError(TEXT("Screenshot Capture Task is not valid."));
			TestStepData->bSuccess = false;
		}
		return true;
	}

	if (TestStepData->ScreenshotTask && TestStepData->ScreenshotTask->IsTaskDone())
	{
		TestStepData->CapturedScreenshotCount++;
		return true;
	}

	return false;
}

bool FInterchangeInterStepPerformTestsAndCollectGarbageCommand::Update()
{
	using namespace InterchangeAutomationTestUtils::Private;

	FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest();

	// Restore Viewport Parameters
	UAutomationBlueprintFunctionLibrary::SetEditorActiveViewportViewMode(TestStepData->CachedScreenshotParameters.ViewMode);
	UAutomationBlueprintFunctionLibrary::SetEditorActiveViewportWireframeOpacity(TestStepData->CachedScreenshotParameters.WireframeOpacity);

	// Reset the Screenshot task so that next step could start a new task for screenshot.
	TestStepData->ScreenshotTask = nullptr;
	TestStepData->bCanTakeScreenshot = false;

	FinishCurrentStep(*TestStepData, CurrentTest);

	if (TestStepData->TestPlanData.InterchangeResults)
	{
		// Populate the automation test execution info with the interchange import results
		for (UInterchangeResult* Result : TestStepData->TestPlanData.InterchangeResults->GetResults())
		{
			switch (Result->GetResultType())
			{
			case EInterchangeResultType::Error:
			{
				if (CurrentTest)
				{
					CurrentTest->AddError(Result->GetText().ToString());
				}
				TestStepData->bSuccess = false;
				break;
			}
			case EInterchangeResultType::Warning:
			{
				if (CurrentTest)
				{
					CurrentTest->AddWarning(Result->GetText().ToString());
				}
				break;
			}
			}
		}
		TestStepData->TestPlanData.InterchangeResults->ClearFlags(RF_Standalone);
	}

	// Finished with the interchange results - null it so it will be GC'd later
	TestStepData->TestPlanData.InterchangeResults = nullptr;

	if (CurrentTest)
	{
		CurrentTest->PopContext();
	}

	// Collect garbage between every step, so that we remove renamed packages which come from the save+reload operation
	// Note we also reset the transaction buffer here to stop it from holding onto references which would prevent garbage collection.
	// @todo: not really a big fan of this; is there a better way of just disabling transactions?
	if ((GEditor != nullptr) && (GEditor->Trans != nullptr))
	{
		GEditor->Trans->Reset(FText::FromString("Discard undo history during Automation testing."));
	}

	{
		TArray<int32> RemoveObjectIndices;
		for (int32 ResultObjectIndex = 0; ResultObjectIndex < TestStepData->TestPlanData.ResultObjects.Num(); ++ResultObjectIndex)
		{
			UObject* ResultObject = TestStepData->TestPlanData.ResultObjects[ResultObjectIndex];

			if (UWorld* World = Cast<UWorld>(ResultObject))
			{
				if (TestStepData->TestPlanData.TestPlan->GetCurrentWorld() == World)
				{
					continue;
				}
				else
				{
					ResultObject->ClearFlags(RF_Standalone);
					RemoveObjectIndices.Add(ResultObjectIndex);
				}
			}
		}

		for (int32 RemoveIndex = RemoveObjectIndices.Num() - 1; RemoveIndex >= 0; --RemoveIndex)
		{
			TestStepData->TestPlanData.ResultObjects.RemoveAt(RemoveObjectIndices[RemoveIndex]);
		}
	}

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

	{
		TArray<int32> RemoveObjectIndices;
		for (int32 ResultObjectIndex = 0; ResultObjectIndex < TestStepData->TestPlanData.ResultObjects.Num(); ++ResultObjectIndex)
		{
			UObject* ResultObject = TestStepData->TestPlanData.ResultObjects[ResultObjectIndex];
			if (ResultObject && ResultObject->HasAllFlags(RF_BeginDestroyed | RF_FinishDestroyed))
			{
				RemoveObjectIndices.Add(ResultObjectIndex);
			}
		}

		for (int32 RemoveIndex = RemoveObjectIndices.Num() - 1; RemoveIndex >= 0; --RemoveIndex)
		{
			TestStepData->TestPlanData.ResultObjects.RemoveAt(RemoveObjectIndices[RemoveIndex]);
		}
	}

	return true;
}

// Called after all the steps.
bool FInterchangeTestCleanUpCommand::Update()
{
	TArray<UObject*> ObjectsToDelete;
	TArray<UWorld*> WorldsToDelete;
	ObjectsToDelete.Reserve(ObjectsToDelete.Max() + TestStepData->TestPlanData.ResultObjects.Num());
	for (UObject* ResultObject : TestStepData->TestPlanData.ResultObjects)
	{
		if (!ResultObject)
		{
			continue;
		}

		if (ResultObject->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
		{
			continue;
		}

		if (UWorld* World = Cast<UWorld>(ResultObject))
		{
			if (TestStepData->TestPlanData.TestPlan->GetCurrentWorld() == World)
			{
				continue;
			}
			else
			{
				WorldsToDelete.Add(World);
			}
		}
		else if (AActor* Actor = Cast<AActor>(ResultObject))
		{
			constexpr bool bShouldModifyLevel = true;
			Actor->GetWorld()->EditorDestroyActor(Actor, bShouldModifyLevel);
			// Call UObject::Rename directly on actor to avoid AActor::Rename which unnecessarily unregister and re-register components
			Actor->UObject::Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
		}
		else
		{
			ObjectsToDelete.Add(ResultObject);
		}
	}

	//Make sure all compilation is done before deleting some objects
	FAssetCompilingManager::Get().FinishAllCompilation();

	for (UWorld* WorldToDelete : WorldsToDelete)
	{
		if (!WorldToDelete->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
		{
			GEngine->DestroyWorldContext(WorldToDelete);
			WorldToDelete->DestroyWorld(true);
		}
	}

	constexpr bool bShowConfirmation = false;
	ObjectTools::ForceDeleteObjects(ObjectsToDelete, bShowConfirmation);

	// Destroy Transient World or reload the custom level
	TestStepData->TestPlanData.TestPlan->CleanupLevel();

	if (!TestStepData->PendingDeleteDirectoryPath.IsEmpty())
	{
		constexpr bool bRequireExists = false;
		constexpr bool bDeleteRecursively = true;
		IFileManager::Get().DeleteDirectory(*TestStepData->PendingDeleteDirectoryPath, bRequireExists, bDeleteRecursively);
	}

	return true;

}

bool FInterchangeTestAutomationTestSuccessCommand::Update()
{
	if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
	{
		CurrentTest->TestTrue(TEXT("Interchange Import Automation Test Success"), TestStepData->bSuccess);
		CurrentTest->TestEqual(TEXT("Interchange Import Automation Test Captured Screenshots"), TestStepData->CapturedScreenshotCount, TestStepData->RequiredScreenshotCount);
	}
	return true;
}