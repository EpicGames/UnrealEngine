// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/AutomationTest.h"
#include "InterchangeImportTestStepBase.h"
#include "InterchangeManager.h"
#include "InterchangeImportTestData.h"

class UAutomationEditorTask;
class UClass;

struct FInterchangeAutomationTestStepData
{
	FString PendingDeleteDirectoryPath; 
	FInterchangeImportTestData TestPlanData;
	FInterchangeTestScreenshotParameters CachedScreenshotParameters;
	TTuple<UE::Interchange::FAssetImportResultPtr, UE::Interchange::FSceneImportResultPtr> Results;
	UAutomationEditorTask* ScreenshotTask = nullptr;
	int32 StepIndex = 0;
	int32 RequiredScreenshotCount = 0;
	int32 CapturedScreenshotCount = 0;
	bool bIsReimportStep = false;
	bool bCanTakeScreenshot = false;
	bool bSuccess = true;
};

using InterchangeAutomationTestStepDataRef = TSharedRef< FInterchangeAutomationTestStepData, ESPMode::ThreadSafe>;

DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FInterchangeIntializeStepCommand, InterchangeAutomationTestStepDataRef, TestStepData, bool, bIsReimport, int32, StepIndex);

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FInterchangeInterStepCollectResultCommand, InterchangeAutomationTestStepDataRef, TestStepData);

DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FInterchangeSetupScreenshotViewportCommand, InterchangeAutomationTestStepDataRef, TestStepData, FInterchangeTestScreenshotParameters, ScreenshotParameters);

DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FInterchangeCaptureScreenshotCommand, InterchangeAutomationTestStepDataRef, TestStepData, FInterchangeTestScreenshotParameters, ScreenshotParameters);

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FInterchangeInterStepPerformTestsAndCollectGarbageCommand, InterchangeAutomationTestStepDataRef, TestStepData);

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FInterchangeTestAutomationTestSuccessCommand, InterchangeAutomationTestStepDataRef, TestStepData);

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FInterchangeTestCleanUpCommand, InterchangeAutomationTestStepDataRef, TestStepData);