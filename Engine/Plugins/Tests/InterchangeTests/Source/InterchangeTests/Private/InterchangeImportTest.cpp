// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetCompilingManager.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "HAL/FileManager.h"
#include "InterchangeAutomatedTestUtils.h"
#include "InterchangeHelper.h"
#include "InterchangeImportTestData.h"
#include "InterchangeImportTestPlan.h"
#include "InterchangeImportTestStepBase.h"
#include "InterchangeImportTestStepImport.h"
#include "InterchangeImportTestStepReimport.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "Tests/AutomationCommon.h"

IMPLEMENT_CUSTOM_COMPLEX_AUTOMATION_TEST(FInterchangeImportTest, FAutomationTestBase, UE::Interchange::FInterchangeImportTestPlanStaticHelpers::GetBeautifiedTestName(), EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

void FInterchangeImportTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	// For now, we can't run Interchange automation tests on Mac, as there's a problem with the InterchangeWorker. For now, do nothing on Mac.
	// @todo: find a solution to this
#if PLATFORM_MAC || PLATFORM_LINUX
	return;
#else
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FAssetData> AllTestPlans;
	AssetRegistryModule.Get().GetAssetsByClass(UInterchangeImportTestPlan::StaticClass()->GetClassPathName(), AllTestPlans, true);

	// Get a list of all paths containing InterchangeTestPlan assets
	TSet<FString> Paths;
	for (const FAssetData& TestPlan : AllTestPlans)
	{
		Paths.Add(TestPlan.GetObjectPathString());
	}

	// And create a sorted list from the set.
	// Each unique path will be a sub-entry in the automated test list.
	// All test plans within a given folder will be executed in parallel.

	Paths.Sort([](const FString& A, const FString& B) { return A < B; });

	for (const FString& Path : Paths)
	{
		using namespace UE::Interchange;
		OutTestCommands.Add(Path);
		OutBeautifiedNames.Add(FInterchangeImportTestPlanStaticHelpers::GetTestNameFromObjectPathString(Path));
	}
#endif
}


bool FInterchangeImportTest::RunTest(const FString& Path)
{
	#define ADD_COMMAND(ClassDeclaration) if (bRunSynchornously) {ClassDeclaration.Update();}\
	else{ADD_LATENT_AUTOMATION_COMMAND(ClassDeclaration)}

	// For now, we can't run Interchange automation tests on Mac, as there's a problem with the InterchangeWorker. For now, do nothing on Mac.
	// @todo: find a solution to this
#if PLATFORM_MAC || PLATFORM_LINUX
	return true;
#else
	
	TestParameterContext = FString::Printf(TEXT("Interchange.%s"), *UE::Interchange::FInterchangeImportTestPlanStaticHelpers::GetTestNameFromObjectPathString(Path));

	static const auto CVarInterchangeFbx = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.Import.FBX"));
	bool IsInterchangeFbxEnabled = CVarInterchangeFbx->GetBool();
	UE::Interchange::FScopedLambda IsInterchangeEnabledGuard([&IsInterchangeFbxEnabled]()
		{
			CVarInterchangeFbx->Set(IsInterchangeFbxEnabled, ECVF_SetByConsole);
		});
	//Make sure interchange is enabled for fbx
	CVarInterchangeFbx->Set(true, ECVF_SetByConsole);

	TSharedPtr<FInterchangeAutomationTestStepData, ESPMode::ThreadSafe> AutomationTestStepData = MakeShared<FInterchangeAutomationTestStepData, ESPMode::ThreadSafe>();
	
	const FName PackageName = FName(FPaths::GetBaseFilename(Path, false));
	const FName PackagePath = FName(FPaths::GetPath(Path));
	const FName AssetName = FName(FPaths::GetBaseFilename(Path, true));
	const FTopLevelAssetPath ClassName = UInterchangeImportTestPlan::StaticClass()->GetClassPathName();
	AutomationTestStepData->TestPlanData = FInterchangeImportTestData(PackageName, PackagePath, AssetName, ClassName);
	AutomationTestStepData->TestPlanData.TestPlan = CastChecked<UInterchangeImportTestPlan>(AutomationTestStepData->TestPlanData.GetAsset());
	
	if (!ensure(AutomationTestStepData->TestPlanData.TestPlan))
	{
		AutomationTestStepData.Reset();
		return false;
	}

	// Base path to import assets to

	const FString SubDirToUse = TEXT("Interchange/Temp/ImportTest/");
	const FString BasePackagePath = FPaths::Combine(TEXT("/Game/Tests"), SubDirToUse);
	const FString BaseFilePath = FPaths::Combine(FPaths::ProjectContentDir(), TEXT("Tests"), SubDirToUse);

	AutomationTestStepData->PendingDeleteDirectoryPath = FPaths::Combine(FPaths::ProjectContentDir(), TEXT("Tests/Interchange/Temp/"));

	// Clear out the folder contents before we do anything else

	constexpr bool bRequireExists = false;
	constexpr bool bDeleteRecursively = true;
	IFileManager::Get().DeleteDirectory(*BaseFilePath, bRequireExists, bDeleteRecursively);

	FString TestPlanAssetName = AutomationTestStepData->TestPlanData.AssetName.ToString().Replace(TEXT("/"), TEXT("_"));
	AutomationTestStepData->TestPlanData.DestAssetPackagePath = BasePackagePath / TestPlanAssetName;
	AutomationTestStepData->TestPlanData.DestAssetFilePath = BaseFilePath / TestPlanAssetName;
	const bool bAddRecursively = true;
	IFileManager::Get().MakeDirectory(*AutomationTestStepData->TestPlanData.DestAssetFilePath, bAddRecursively);

	AutomationTestStepData->StepIndex = 0;
	AutomationTestStepData->bSuccess = true;

	const bool bRunSynchornously = AutomationTestStepData->TestPlanData.TestPlan->IsRunningSynchornously();

	// Add an expected message so that Level Hierarchies other than Level Actors should not result in a failed test.
	// Occurrences are set to -1 to silenty ignore the error messages.
	constexpr int32 Occurences = -1;
	AddExpectedMessage("Soft references (.+) which does not exist", ELogVerbosity::Error, EAutomationExpectedMessageFlags::Contains, Occurences);
	
	// If it is a level import, custom level or transient world is made available for the import.
	AutomationTestStepData->TestPlanData.TestPlan->SetupLevelForImport();
	
	// Import
	if (UInterchangeImportTestStepImport* TestStepImport = AutomationTestStepData->TestPlanData.TestPlan->ImportStep)
	{
		constexpr bool bIsReimportStep = false;
		ADD_COMMAND(FInterchangeIntializeStepCommand(AutomationTestStepData.ToSharedRef(), bIsReimportStep, -1));
		ADD_COMMAND(FInterchangeInterStepCollectResultCommand(AutomationTestStepData.ToSharedRef()));

		// Screenshot Tests should only be performed via Test Automation Window
		if (!bRunSynchornously && TestStepImport->HasScreenshotTest())
		{
			// Wait so that all the materials applied to actors would be visible in the screenshot.
			ADD_LATENT_AUTOMATION_COMMAND(FWaitForShadersToFinishCompilingInGame());

			const FInterchangeTestScreenshotParameters ScreenshotParameters = TestStepImport->GetScreenshotParameters();

			// Focuses camera on relevant actors in the scene.
			ADD_LATENT_AUTOMATION_COMMAND(FInterchangeSetupScreenshotViewportCommand(AutomationTestStepData.ToSharedRef(), ScreenshotParameters));

			// Camera Viewport Transition Time.
			ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.3f));

			// Send a request for screenshot and wait until comparison is finished.
			ADD_LATENT_AUTOMATION_COMMAND(FInterchangeCaptureScreenshotCommand(AutomationTestStepData.ToSharedRef(), ScreenshotParameters));
		}

		ADD_COMMAND(FInterchangeInterStepPerformTestsAndCollectGarbageCommand(AutomationTestStepData.ToSharedRef()));
	}

	// Reimport 
	{
		constexpr bool bIsReimportStep = true;
		for (int32 TestStepIndex = 0; TestStepIndex < AutomationTestStepData->TestPlanData.TestPlan->ReimportStack.Num(); ++TestStepIndex)
		{
			if (UInterchangeImportTestStepReimport* ReimportTestStep = AutomationTestStepData->TestPlanData.TestPlan->ReimportStack[TestStepIndex])
			{
				ADD_COMMAND(FInterchangeIntializeStepCommand(AutomationTestStepData.ToSharedRef(), bIsReimportStep, TestStepIndex));
				ADD_COMMAND(FInterchangeInterStepCollectResultCommand(AutomationTestStepData.ToSharedRef()));

				// Screenshot Tests should only be performed via Test Automation Window
				if (!bRunSynchornously && ReimportTestStep->HasScreenshotTest())
				{
					// Wait so that all the materials applied to actors would be visible in the screenshot.
					ADD_LATENT_AUTOMATION_COMMAND(FWaitForShadersToFinishCompilingInGame());

					const FInterchangeTestScreenshotParameters ScreenshotParameters = ReimportTestStep->GetScreenshotParameters();

					// Focuses camera on relevant actors in the scene.
					ADD_LATENT_AUTOMATION_COMMAND(FInterchangeSetupScreenshotViewportCommand(AutomationTestStepData.ToSharedRef(), ScreenshotParameters));

					// Camera Viewport Transition Time.
					ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(0.3f));

					// Send a request for screenshot and wait until comparison is finished.
					ADD_LATENT_AUTOMATION_COMMAND(FInterchangeCaptureScreenshotCommand(AutomationTestStepData.ToSharedRef(), ScreenshotParameters));
				}

				ADD_COMMAND(FInterchangeInterStepPerformTestsAndCollectGarbageCommand(AutomationTestStepData.ToSharedRef()));
			}
		}
	}
	ADD_COMMAND(FInterchangeTestCleanUpCommand(AutomationTestStepData.ToSharedRef()));
	ADD_COMMAND(FInterchangeTestAutomationTestSuccessCommand(AutomationTestStepData.ToSharedRef()));

	return true;

#endif
#undef ADD_COMMAND
}
