// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeImportTestPlan.h"

#include "Editor.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "InterchangeImportTestStepImport.h"
#include "InterchangeImportTestStepReimport.h"
#include "JsonObjectConverter.h"
#include "LevelEditorSubsystem.h"
#include "Logging/MessageLog.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeImportTestPlan)

#define LOCTEXT_NAMESPACE "InterchangeImportTestPlan"

#if WITH_EDITOR
#include "HAL/IConsoleManager.h"

static bool GInterchangeTestPlanPersistWarningDialogSuppresion = true;
static FAutoConsoleVariableRef CCvarInterchangeDefaultShowEssentialsView(
	TEXT("Interchange.TestPlan.PersistWarningDialogSuppression"),
	GInterchangeTestPlanPersistWarningDialogSuppresion,
	TEXT("Whether the suppresion choices for warning dialogs in interchange test plan persisted beyond the current session."),
	ECVF_Default);


FSuppressableWarningDialog::EMode UInterchangeImportTestPlan::GetInterchangeTestPlanWarningDialogMode()
{
	return GInterchangeTestPlanPersistWarningDialogSuppresion ? FSuppressableWarningDialog::EMode::Default : FSuppressableWarningDialog::EMode::DontPersistSuppressionAcrossSessions;
}
#endif


UInterchangeImportTestPlan::UInterchangeImportTestPlan()
{
	ImportStep = CreateDefaultSubobject<UInterchangeImportTestStepImport>("ImportStep");
	ImportStep->ParentTestPlan = this;
	WorldPath = FSoftObjectPath(TEXT("/Game/Tests/Interchange/InterchangeTestMap.InterchangeTestMap"));
}

bool UInterchangeImportTestPlan::ReadFromJson(const FString& Filename)
{
	FString JsonString;
	if (!FFileHelper::LoadFileToString(JsonString, *Filename))
	{
		return false;
	}

	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(JsonString);
	if (!FJsonSerializer::Deserialize(JsonReader, JsonObject) || !JsonObject.IsValid())
	{
		return false;
	}

	if (!FJsonObjectConverter::JsonObjectToUStruct(JsonObject.ToSharedRef(), UInterchangeImportTestPlan::StaticClass(), this))
	{
		return false;
	}

	return true;
}


void UInterchangeImportTestPlan::WriteToJson(const FString& Filename)
{
	FString JsonString;
	if (!FJsonObjectConverter::UStructToJsonObjectString(UInterchangeImportTestPlan::StaticClass(), this, JsonString))
	{
		return;
	}

	FFileHelper::SaveStringToFile(JsonString, *Filename);
}


void UInterchangeImportTestPlan::RunThisTest()
{
	bRunSynchronously = true;
	FMessageLog AutomationEditorLog("AutomationTestingLog");
	FString NewPageName = FString::Printf(TEXT("----- Interchange Import Test: %s----"), *GetPathName());
	FText NewPageNameText = FText::FromString(*NewPageName);
	AutomationEditorLog.Open();
	AutomationEditorLog.NewPage(NewPageNameText);
	AutomationEditorLog.Info(NewPageNameText);

	FAutomationTestFramework& TestFramework = FAutomationTestFramework::Get();

	TestFramework.StartTestByName(FString(TEXT("FInterchangeImportTest ")) + GetPathName(), 0);

	FAutomationTestExecutionInfo ExecutionInfo;
	if (TestFramework.StopTest(ExecutionInfo))
	{
		AutomationEditorLog.Info(LOCTEXT("TestPassed", "Passed"));
	}
	else
	{
		for (const auto& Entry : ExecutionInfo.GetEntries())
		{
			switch (Entry.Event.Type)
			{
			case EAutomationEventType::Error:
				AutomationEditorLog.Error(FText::FromString(Entry.ToString()));
				break;

			case EAutomationEventType::Warning:
				AutomationEditorLog.Warning(FText::FromString(Entry.ToString()));
				break;

			case EAutomationEventType::Info:
				AutomationEditorLog.Info(FText::FromString(Entry.ToString()));
				break;
			}
		}
	}

	bRunSynchronously = false;
}

bool UInterchangeImportTestPlan::IsRunningSynchornously() const
{
	return bRunSynchronously;
}


void UInterchangeImportTestPlan::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	if (HasAnyFlags(RF_WasLoaded))
	{
		if (!Steps_DEPRECATED.IsEmpty())
		{
			bool bAssignedImportStep = false;
			UInterchangeImportTestStepImport* FirstValidImportTestStep = nullptr;
			// Because we are checking consecutive import and reimport steps.
			for (int32 StepIndex = 0; StepIndex < Steps_DEPRECATED.Num(); ++StepIndex)
			{
				if (UInterchangeImportTestStepImport* ImportTestStep = Cast<UInterchangeImportTestStepImport>(Steps_DEPRECATED[StepIndex]))
				{
					if (!FirstValidImportTestStep)
					{
						FirstValidImportTestStep = ImportTestStep;
					}
					if (bAssignedImportStep)
					{
						break;
					}
				}

				if (!bAssignedImportStep && (StepIndex < Steps_DEPRECATED.Num() - 1))
				{
					UInterchangeImportTestStepImport* ImportTestStep = Cast<UInterchangeImportTestStepImport>(Steps_DEPRECATED[StepIndex]);
					UInterchangeImportTestStepReimport* ReimportTestStep = Cast<UInterchangeImportTestStepReimport>(Steps_DEPRECATED[StepIndex + 1]);
					if (ImportTestStep && ReimportTestStep)
					{
						ImportStep = ImportTestStep;

						bAssignedImportStep = true;
					}
				}

				if (UInterchangeImportTestStepReimport* ReimportTestStep = Cast<UInterchangeImportTestStepReimport>(Steps_DEPRECATED[StepIndex]))
				{
					if (bAssignedImportStep)
					{
						ReimportStack.Add(ReimportTestStep);
					}
				}
			}

			// The Test Plan only contained Import Steps.
			if (ReimportStack.IsEmpty() && FirstValidImportTestStep)
			{
				ImportStep = FirstValidImportTestStep;
			}
			Steps_DEPRECATED.Empty();
			bChangeObjectOuters = true;
		}
		FCoreUObjectDelegates::OnAssetLoaded.AddUObject(this, &UInterchangeImportTestPlan::OnAssetLoaded);
	}
#endif
}

void UInterchangeImportTestPlan::OnAssetLoaded(UObject* Asset)
{
	if (Asset == this)
	{
		if (bChangeObjectOuters)
		{
			ImportStep->Rename(nullptr, this, REN_NonTransactional | REN_DontCreateRedirectors);
		}
		ImportStep->ParentTestPlan = this;

		for (const TObjectPtr<UInterchangeImportTestStepReimport>& ReimportTestStep : ReimportStack)
		{
			if (bChangeObjectOuters)
			{
				ReimportTestStep->Rename(nullptr, this, REN_NonTransactional | REN_DontCreateRedirectors);
			}
			ReimportTestStep->InitializeReimportStepFromImportStep(ImportStep);
			ReimportTestStep->ParentTestPlan = this;
		}
		
		MarkPackageDirty();
		// Unregister to the OnAssetLoad event as it is not needed anymore
		FCoreUObjectDelegates::OnAssetLoaded.RemoveAll(this);
	}
}

#if WITH_EDITOR

void UInterchangeImportTestPlan::PreEditUndo()
{
	PrevReimportStackSize = ReimportStack.Num();
}

void UInterchangeImportTestPlan::PostEditUndo()
{
	if (PrevReimportStackSize != ReimportStack.Num())
	{
		for (int32 ReimportStepIndex = 0; ReimportStepIndex < ReimportStack.Num(); ++ReimportStepIndex)
		{
			if (ReimportStack[ReimportStepIndex] != nullptr)
			{
				ReimportStack[ReimportStepIndex]->RemoveImportStepPipelineSettingsModifiedDelegate();
				ReimportStack[ReimportStepIndex]->InitializeReimportStepFromImportStep(ImportStep);
			}
		}
		PrevReimportStackSize = ReimportStack.Num();
	}
}

void UInterchangeImportTestPlan::PreEditChange(FProperty* PropertyAboutToChange)
{
	if (PropertyAboutToChange && (PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UInterchangeImportTestPlan, ReimportStack)))
	{
		for (int32 ReimportStepIndex = 0; ReimportStepIndex < ReimportStack.Num(); ++ReimportStepIndex)
		{
			if (ReimportStack[ReimportStepIndex] != nullptr)
			{
				ReimportStack[ReimportStepIndex]->RemoveImportStepPipelineSettingsModifiedDelegate();
			}
		}
	}
	Super::PreEditChange(PropertyAboutToChange);
}

void UInterchangeImportTestPlan::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UInterchangeImportTestPlan, ReimportStack))
	{
		for (int32 ReimportStepIndex = 0; ReimportStepIndex < ReimportStack.Num(); ++ReimportStepIndex)
		{
			if (ReimportStack[ReimportStepIndex] != nullptr && !ImportStep->OnImportTestStepDataChanged.IsBoundToObject(ReimportStack[ReimportStepIndex]))
			{
				ReimportStack[ReimportStepIndex]->InitializeReimportStepFromImportStep(ImportStep);
			}
		}
	}
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif

void UInterchangeImportTestPlan::SetupLevelForImport()
{
	if (!ImportStep->bImportIntoLevel)
	{
		return;
	}

	// Create transient world to host data from producer
	if (!IsRunningSynchornously() && WorldPath.IsValid())
	{
		if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
		{
			LevelEditorSubsystem->LoadLevel(WorldPath.GetAssetPathString());
		}
	}
	else
	{
		TransientWorld = TStrongObjectPtr<UWorld>(NewObject<UWorld>(GetTransientPackage()));
		TransientWorld->WorldType = EWorldType::EditorPreview;

		FWorldContext& WorldContext = GEngine->CreateNewWorldContext(TransientWorld->WorldType);
		WorldContext.SetCurrentWorld(TransientWorld.Get());

		TransientWorld->InitializeNewWorld(UWorld::InitializationValues()
			.AllowAudioPlayback(false)
			.CreatePhysicsScene(false)
			.RequiresHitProxies(false)
			.CreateNavigation(false)
			.CreateAISystem(false)
			.ShouldSimulatePhysics(false)
			.SetTransactional(false)
		);

		UE_LOGF(LogTemp, Display, "Test Plan World Path Name: %ls", *(TransientWorld.Get()->GetPathName()));
	}
}

void UInterchangeImportTestPlan::CleanupLevel()
{
	if (!ImportStep->bImportIntoLevel)
	{
		return;
	}

	if (!IsRunningSynchornously() && WorldPath.IsValid())
	{
		if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
		{
			LevelEditorSubsystem->LoadLevel(WorldPath.GetAssetPathString());
		}
	}
	else if (TransientWorld)
	{
		// Now delete world
		GEngine->DestroyWorldContext(TransientWorld.Get());
		TransientWorld->DestroyWorld(true);
		TransientWorld.Reset();
	}
}

namespace UE::Interchange
{
	FString FInterchangeImportTestPlanStaticHelpers::GetTestNameFromObjectPathString(const FString& InObjectPathString, bool bAddBeautifiedTestNamePrefix /*= false*/)
	{
		FString PackagePath, ObjectName;
		InObjectPathString.Split(TEXT("."), &PackagePath, &ObjectName);

		FString BeautifiedName = PackagePath;
		FPaths::MakePathRelativeTo(BeautifiedName, *GetInterchangeImportTestRootGameFolder());
		BeautifiedName.ReplaceCharInline(TEXT('/'), TEXT('.'));
		BeautifiedName.ReplaceCharInline(TEXT('\\'), TEXT('.'));

		if (bAddBeautifiedTestNamePrefix)
		{
			return FString::Printf(TEXT("%s.%s"), *GetBeautifiedTestName(), *BeautifiedName);
		}

		return BeautifiedName;
	}

	FString FInterchangeImportTestPlanStaticHelpers::GetBeautifiedTestName()
	{
		return TEXT("Editor.Interchange");
	}

	FString FInterchangeImportTestPlanStaticHelpers::GetInterchangeImportTestRootGameFolder()
	{
		return TEXT("/Game/Tests/Interchange/");
	}
}
#undef LOCTEXT_NAMESPACE