// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeImportTestStepBase.h"

#include "AssetCompilingManager.h"
#include "ComponentReregisterContext.h"
#include "InterchangeImportTestData.h"
#include "InterchangeTestFunction.h"
#include "Interfaces/Interface_AsyncCompilation.h"
#include "Misc/AutomationTest.h"
#include "UObject/SavePackage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeImportTestStepBase)


bool UInterchangeImportTestStepBase::PerformTests(FInterchangeImportTestData& Data, FAutomationTestBase* CurrentTest)
{
	// Add the Interchange results container to the list of UObject* we pass to the test functions.
	// This way we can match against tests which can operate on the UInterchangeResultsContainer too.
	TArray<UObject*> ResultObjects = Data.ResultObjects;
	ResultObjects.Add(Data.InterchangeResults);

	bool bSuccess = true;

	for (FInterchangeTestFunction& Test : Tests)
	{
		FInterchangeTestFunctionResult Result = Test.Invoke(ResultObjects);
		
		for (const FString& Warning : Result.GetWarnings())
		{
			if (CurrentTest)
			{
				CurrentTest->AddWarning(Warning);
			}
		}

		for (const FString& Error : Result.GetErrors())
		{
			if (CurrentTest)
			{
				CurrentTest->AddError(Error);
			}
		}

		bSuccess &= Result.IsSuccess();
	}

	return bSuccess;
}


void UInterchangeImportTestStepBase::SaveReloadAssets(FInterchangeImportTestData& Data)
{
	// First save
	for (const FAssetData& AssetData : Data.ImportedAssets)
	{
		const bool bLoadAsset = false;
		UObject* AssetObject = AssetData.FastGetAsset(bLoadAsset);
		if (!AssetObject)
		{
			continue;
		}
		UPackage* PackageObject = AssetObject->GetPackage();
		if (!PackageObject)
		{
			continue;
		}
		AssetObject->MarkPackageDirty();
		FSavePackageArgs SaveArgs;
		SaveArgs.TopLevelFlags = RF_Standalone;
		SaveArgs.SaveFlags = SAVE_NoError;
		UPackage::SavePackage(PackageObject, AssetObject,
			*FPackageName::LongPackageNameToFilename(AssetData.PackageName.ToString(), FPackageName::GetAssetPackageExtension()),
			SaveArgs);
	}

	// Then rename original objects and their packages, and mark as garbage
	for (const FAssetData& AssetData : Data.ImportedAssets)
	{
		const bool bLoadAsset = false;
		UObject* AssetObject = AssetData.FastGetAsset(bLoadAsset);
		if (!AssetObject)
		{
			continue;
		}
		//Make sure asset compilation is done before renaming them and mark them for garbage collect
		if (const IInterface_AsyncCompilation* AsyncAsset = Cast<IInterface_AsyncCompilation>(AssetObject))
		{
			if (AsyncAsset->IsCompiling())
			{
				FAssetCompilingManager::Get().FinishCompilationForObjects({AssetObject});
			}
		}
		UPackage* PackageObject = AssetObject->GetPackage();
		if (!PackageObject || !ensure(PackageObject == AssetData.GetPackage()))
		{
			continue;
		}

		// Mark all objects in the package as garbage, and remove the standalone flag, so that GC can remove the package later
		TArray<UObject*> ObjectsInPackage;
		GetObjectsWithPackage(PackageObject, ObjectsInPackage, EGetObjectsFlags::IncludeNestedObjects);
		for (UObject* ObjectInPackage : ObjectsInPackage)
		{
			if (!ObjectInPackage)
			{
				continue;
			}
			ObjectInPackage->ClearFlags(RF_Standalone | RF_Public);
			ObjectInPackage->MarkAsGarbage();
		}

		// Renaming the original objects avoids having to do a GC sweep here (this is done at the end of each test step)
		// Any existing references to them will be retained but irrelevant.
		// Then the new object can be loaded in their place, as if it were being loaded for the first time.
		const ERenameFlags RenameFlags = REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty | REN_AllowPackageLinkerMismatch;
		PackageObject->Rename(*(PackageObject->GetName() + TEXT("_TRASH")), nullptr, RenameFlags);
		PackageObject->RemoveFromRoot();
		PackageObject->MarkAsGarbage();

		// Remove the old version of the asset object from the results
		Data.ResultObjects.Remove(AssetObject);
	}

	//Garbage collect objects
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

	// Now reload
	for (const FAssetData& AssetData : Data.ImportedAssets)
	{
		if (!ensure(!AssetData.IsAssetLoaded()))
		{
			continue;
		}
		UObject* AssetObject = AssetData.GetAsset();
		if (!AssetObject)
		{
			continue;
		}
		Data.ResultObjects.Add(AssetObject);
		if (const IInterface_AsyncCompilation* AsyncAsset = Cast<IInterface_AsyncCompilation>(AssetObject))
		{
			if (AsyncAsset->IsCompiling())
			{
				FAssetCompilingManager::Get().FinishCompilationForObjects({ AssetObject });
			}
		}
	}
}
