// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeImportTestStepImport.h"

#include "Dialogs/Dialogs.h"
#include "GameFramework/WorldSettings.h"
#include "HAL/FileManager.h"
#include "InterchangeImportTestData.h"
#include "InterchangeImportTestPlan.h"
#include "InterchangeManager.h"
#include "InterchangeProjectSettings.h"
#include "InterchangeTestsLog.h"
#include "InterchangeTranslatorBase.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeImportTestStepImport)

#define LOCTEXT_NAMESPACE "InterchangeImportTestStepImport"

UInterchangeImportTestStepImport::UInterchangeImportTestStepImport()
{
	PipelineSettings.ParentTestStep = this;
}

TTuple<UE::Interchange::FAssetImportResultPtr, UE::Interchange::FSceneImportResultPtr> UInterchangeImportTestStepImport::StartStep(FInterchangeImportTestData& Data)
{
	// Empty the destination folder here if requested
	if (bEmptyDestinationFolderPriorToImport)
	{
		for (UObject* AssetObject : Data.ResultObjects)
		{
			UPackage* PackageObject = AssetObject->GetPackage();
			if (!ensure(PackageObject))
			{
				continue;
			}
			// Mark all objects in the package as garbage, and remove the standalone flag, so that GC can remove the temporary asset later
			//Also rename them so we dont found them if we re-import the same file at the same place
			TArray<UObject*> ObjectsInPackage;
			GetObjectsWithPackage(PackageObject, ObjectsInPackage, EGetObjectsFlags::IncludeNestedObjects);
			for (UObject* ObjectInPackage : ObjectsInPackage)
			{
				//Do not rename actors
				if (!ObjectInPackage->IsA<AActor>())
				{
					const ERenameFlags RenameFlags = REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty | REN_AllowPackageLinkerMismatch;
					ObjectInPackage->Rename(nullptr, nullptr, RenameFlags);
				}
				ObjectInPackage->ClearFlags(RF_Standalone | RF_Public);
				ObjectInPackage->MarkAsGarbage();
			}
		}

		Data.ResultObjects.Empty();
		Data.ImportedAssets.Empty();

		const bool bRequireExists = true;
		const bool bDeleteRecursively = true;
		IFileManager::Get().DeleteDirectory(*Data.DestAssetFilePath, bRequireExists, bDeleteRecursively);

		const bool bAddRecursively = true;
		IFileManager::Get().MakeDirectory(*Data.DestAssetFilePath, bAddRecursively);

		// @todo: Is there a better way of deleting all the files inside a directory without also deleting the directory?
	}

	// Start the Interchange import
	UE::Interchange::FScopedSourceData ScopedSourceData(SourceFile.FilePath);

	FImportAssetParameters Params;
	if (bUseOverridePipelineStack)
	{
		Params.OverridePipelines.Reserve(PipelineStack.Num());
		for (TObjectPtr<UInterchangePipelineBase> Pipeline : PipelineStack)
		{
			Params.OverridePipelines.Add(Pipeline);
		}
	}
	else if (PipelineSettings.CustomPipelines.Num() > 0)
	{
		Params.OverridePipelines.Reserve(PipelineSettings.CustomPipelines.Num());
		for (TObjectPtr<UInterchangePipelineBase> Pipeline : PipelineSettings.CustomPipelines)
		{
			Params.OverridePipelines.Add(Pipeline);
		}
	}
	Params.bIsAutomated = true;

	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();

	if (bImportIntoLevel)
	{
		// Use the world from the Test Plan for Level Import 
		if (ensure(Data.TestPlan))
		{
			Params.ImportLevel = Data.TestPlan->GetCurrentLevel();
		}
		return InterchangeManager.ImportSceneAsync(Data.DestAssetPackagePath, ScopedSourceData.GetSourceData(), Params);
	}
	else
	{
		UE::Interchange::FAssetImportResultPtr AssetImportResult = InterchangeManager.ImportAssetAsync(Data.DestAssetPackagePath, ScopedSourceData.GetSourceData(), Params);
		return {AssetImportResult, nullptr};
	}
}

FTestStepResults UInterchangeImportTestStepImport::FinishStep(FInterchangeImportTestData& Data, FAutomationTestBase* CurrentTest)
{
	FTestStepResults Results;

	// Run all the tests
	bool bSuccess = PerformTests(Data, CurrentTest);
	
	// Only keep assets as result objects since the world and its actors are being destroyed
	TArray<UObject*> ResultObjects = MoveTemp(Data.ResultObjects);
	Data.ResultObjects.Reset(ResultObjects.Num());

	for (UObject* Object : ResultObjects)
	{
		if (!Object->IsA<AActor>())
		{
			Data.ResultObjects.Add(Object);
		}
	}

	Results.bTestStepSuccess = bSuccess;
	return Results;
}

FString UInterchangeImportTestStepImport::GetContextString() const
{
	return FString(TEXT("Importing ")) + FPaths::GetCleanFilename(SourceFile.FilePath);
}

bool UInterchangeImportTestStepImport::HasScreenshotTest() const
{
	return bImportIntoLevel && bTakeScreenshot;
}

FInterchangeTestScreenshotParameters UInterchangeImportTestStepImport::GetScreenshotParameters() const
{
	return ScreenshotParameters;
}

bool UInterchangeImportTestStepImport::CanEditPipelineSettings() const
{
	if (SourceFile.FilePath.IsEmpty())
	{
		return false;
	}

	if (bUseOverridePipelineStack)
	{
		if (PipelineStack.IsEmpty())
		{
			return false;
		}

		for (const UInterchangePipelineBase* Pipeline : PipelineStack)
		{
			if (!Pipeline)
			{
				return false;
			}
		}
	}

	return true;
}

void UInterchangeImportTestStepImport::EditPipelineSettings()
{
#if WITH_EDITOR
	if (SourceFile.FilePath.IsEmpty())
	{
		return;
	}

	UE::Interchange::FScopedSourceData ScopedSourceData(SourceFile.FilePath);
	UE::Interchange::FScopedTranslator ScopedTranslator(ScopedSourceData.GetSourceData());
	if (!ScopedTranslator.GetTranslator())
	{
		UE_LOGF(LogInterchangeTests, Error, "Cannot import file. The source data is not supported. Try enabling the [%ls] extension for Interchange.", *FPaths::GetExtension(ScopedSourceData.GetSourceData()->GetFilename()));
		return;
	}

	const FInterchangeImportSettings& InterchangeImportSettings = FInterchangeProjectSettingsUtils::GetDefaultImportSettings(bImportIntoLevel);
	if (InterchangeImportSettings.PipelineStacks.Num() == 0)
	{
		UE_LOGF(LogInterchangeTests, Error, "Failed to configure pipelines. There is no pipeline stack defined for the content import type.");
		return;
	}

	if (!InterchangeImportSettings.PipelineStacks.Contains(InterchangeImportSettings.DefaultPipelineStack))
	{
		FInterchangeImportSettings& MutableInterchangeImportSettings = FInterchangeProjectSettingsUtils::GetMutableDefaultImportSettings(bImportIntoLevel);
		TArray<FName> Keys;
		MutableInterchangeImportSettings.PipelineStacks.GetKeys(Keys);
		check(Keys.Num() > 0);
		MutableInterchangeImportSettings.DefaultPipelineStack = Keys[0];
	}

	UE::Interchange::FScopedBaseNodeContainer ScopedBaseNodeContainer;

	UInterchangePipelineConfigurationBase* RegisteredPipelineConfiguration = nullptr;
	TSoftClassPtr <UInterchangePipelineConfigurationBase> ImportDialogClass = InterchangeImportSettings.ImportDialogClass;
	if (ImportDialogClass.IsValid())
	{
		UClass* PipelineConfigurationClass = ImportDialogClass.LoadSynchronous();
		if (PipelineConfigurationClass)
		{
			RegisteredPipelineConfiguration = NewObject<UInterchangePipelineConfigurationBase>(GetTransientPackage(), PipelineConfigurationClass, NAME_None, RF_NoFlags);
			if (RegisteredPipelineConfiguration == nullptr)
			{
				UE_LOGF(LogInterchangeTests, Error, "Failed to create a pipeline configuration object.");
				return;
			}
		}
	}

	auto AdjustPipelineSettingForContext = [this, &ScopedBaseNodeContainer](UInterchangePipelineBase* Pipeline)
		{
			EInterchangePipelineContext Context;
			Context = bImportIntoLevel ? EInterchangePipelineContext::SceneImport : EInterchangePipelineContext::AssetImport;
			FInterchangePipelineContextParams ContextParams;
			ContextParams.ContextType = Context;
			ContextParams.BaseNodeContainer = ScopedBaseNodeContainer.GetBaseNodeContainer();
			Pipeline->SetFromReimportOrOverride(bUseOverridePipelineStack);
			Pipeline->AdjustSettingsForContext(ContextParams);
		};

	const bool bCanTranslate = UInterchangeManager::GetInterchangeManager().CanUseTranslator(ScopedTranslator.GetTranslator());
	if (bCanTranslate)
	{
		FScopedSlowTask Progress(2.f, LOCTEXT("TranslatingSourceFile...", "Translating source file..."));
		Progress.MakeDialog();
		Progress.EnterProgressFrame(1.f);

		UInterchangeBaseNodeContainer& BaseNodeContainer = *(ScopedBaseNodeContainer.GetBaseNodeContainer());
		ScopedTranslator.GetTranslator()->SetResultsContainer(NewObject<UInterchangeResultsContainer>(GetTransientPackage(), NAME_None));
		ScopedTranslator.GetTranslator()->Translate(BaseNodeContainer);

		Progress.EnterProgressFrame(1.f);
	}

	const TMap<FName, FInterchangePipelineStack>& DefaultPipelineStacks = InterchangeImportSettings.PipelineStacks;
	TArray<FInterchangeStackInfo> InPipelineStacks;
	TArray<UInterchangePipelineBase*> OutPipelines;

	if (bUseOverridePipelineStack && !PipelineStack.IsEmpty())
	{
		FInterchangeStackInfo& StackInfo = InPipelineStacks.AddDefaulted_GetRef();
		StackInfo.StackName = FName(TEXT("OverridePipelineStack"));
		for (int32 PipelineClassIndex = 0; PipelineClassIndex < PipelineStack.Num(); ++PipelineClassIndex)
		{
			if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance(PipelineStack[PipelineClassIndex]))
			{
				AdjustPipelineSettingForContext(GeneratedPipeline);
				StackInfo.Pipelines.Add(GeneratedPipeline);
			}
		}

		if (StackInfo.Pipelines.IsEmpty())
		{
			UE_LOGF(LogInterchangeTests, Error, "Failed to configure pipelines. There are no pipelines in the override stack");
			return;
		}
	}
	else if (PipelineSettings.CustomPipelines.Num() > 0)
	{
		FInterchangeStackInfo& StackInfo = InPipelineStacks.AddDefaulted_GetRef();
		StackInfo.StackName = FName(TEXT("CustomPipelineStack"));
		for (int32 PipelineIndex = 0; PipelineIndex < PipelineSettings.CustomPipelines.Num(); ++PipelineIndex)
		{
			if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance(PipelineSettings.CustomPipelines[PipelineIndex]))
			{
				AdjustPipelineSettingForContext(GeneratedPipeline);
				StackInfo.Pipelines.Add(GeneratedPipeline);
			}
		}
	}

	for (const TPair<FName, FInterchangePipelineStack>& PipelineStackInfo : DefaultPipelineStacks)
	{
		FName StackName = PipelineStackInfo.Key;
		FInterchangeStackInfo& StackInfo = InPipelineStacks.AddDefaulted_GetRef();
		StackInfo.StackName = StackName;

		const FInterchangePipelineStack& DefPipelineStack = PipelineStackInfo.Value;
		const TArray<FSoftObjectPath>* Pipelines = &DefPipelineStack.Pipelines;

		// If applicable, check to see if a specific pipeline stack is associated with this translator
		for (const FInterchangeTranslatorPipelines& TranslatorPipelines : DefPipelineStack.PerTranslatorPipelines)
		{
			const UClass* TranslatorClass = TranslatorPipelines.Translator.LoadSynchronous();
			if (ScopedTranslator.GetTranslator() && ScopedTranslator.GetTranslator()->IsA(TranslatorClass))
			{
				Pipelines = &TranslatorPipelines.Pipelines;
				break;
			}
		}

		for (int32 PipelineIndex = 0; PipelineIndex < Pipelines->Num(); ++PipelineIndex)
		{
			if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance((*Pipelines)[PipelineIndex]))
			{
				AdjustPipelineSettingForContext(GeneratedPipeline);
				StackInfo.Pipelines.Add(GeneratedPipeline);
			}
		}
	}

	//Show the dialog, a plugin should have register this dialog. We use a plugin to be able to use editor code when doing UI
	EInterchangePipelineConfigurationDialogResult DialogResult = RegisteredPipelineConfiguration->ScriptedShowTestPlanConfigurationDialog(InPipelineStacks, OutPipelines, ScopedSourceData.GetSourceData(), ScopedTranslator.GetTranslator(), ScopedBaseNodeContainer.GetBaseNodeContainer(), nullptr, bImportIntoLevel);

	if (DialogResult == EInterchangePipelineConfigurationDialogResult::SaveConfig)
	{
		if (bUseOverridePipelineStack)
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("UpdatePipelineSettings", "Update Pipeline Settings"));
			Modify();

			PipelineStack.Empty(OutPipelines.Num());
			for (UInterchangePipelineBase* Pipeline : OutPipelines)
			{
				if (UInterchangePipelineBase* InstancedPipeline = DuplicateObject<UInterchangePipelineBase>(Pipeline, this))
				{
					PipelineStack.Add(InstancedPipeline);
				}
			}
		}
		else
		{
			PipelineSettings.UpdatePipelines(OutPipelines);
		}

		BroadcastImportStepChangedEvent(EImportStepDataChangeType::PipelineSettings);
	}
#endif 
}

void UInterchangeImportTestStepImport::ClearPipelineSettings()
{
#if WITH_EDITOR
	if (bUseOverridePipelineStack)
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("ClearPipelineSettings", "Clear Pipeline Settings"));
		Modify();
		PipelineStack.Empty();
	}
	else
	{
		PipelineSettings.ClearPipelines();
	}
	
	BroadcastImportStepChangedEvent(EImportStepDataChangeType::PipelineSettings);
#endif
}

bool UInterchangeImportTestStepImport::IsUsingOverridePipelines(bool bCheckForValidPipelines) const
{
	if (!bCheckForValidPipelines)
	{
		return bUseOverridePipelineStack;
	}

	if (bUseOverridePipelineStack)
	{
		for (const TObjectPtr<UInterchangePipelineBase>& Pipeline : PipelineStack)
		{
			if (!Pipeline)
			{
				return false;
			}
		}
		return true;
	}

	return false;
}

bool UInterchangeImportTestStepImport::ShouldImportIntoLevelChangeRequireMessageBox() const
{
	return !bUseOverridePipelineStack && PipelineSettings.IsUsingModifiedSettings();
}

void UInterchangeImportTestStepImport::PostLoad()
{
	Super::PostLoad();
	if (!SourceFile.FilePath.IsEmpty() && LastSourceFileExtension.IsEmpty())
	{
		LastSourceFileExtension = FPaths::GetExtension(SourceFile.FilePath);
	}
}

#if WITH_EDITOR
void UInterchangeImportTestStepImport::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName UseOverridePipelineStackPropertyName = GET_MEMBER_NAME_CHECKED(UInterchangeImportTestStepImport, bUseOverridePipelineStack);
	static const FName ImportIntoLevelPropertyName = GET_MEMBER_NAME_CHECKED(UInterchangeImportTestStepImport, bImportIntoLevel);

	if (PropertyChangedEvent.Property->GetFName() == UseOverridePipelineStackPropertyName)
	{
		bool bSendPipelineSettingsChangedEvent = false;
		if (bUseOverridePipelineStack && !PipelineSettings.CustomPipelines.IsEmpty())
		{
			FSuppressableWarningDialog::FSetupInfo DialogSetupInfo = FSuppressableWarningDialog::FSetupInfo(
				LOCTEXT("UseOverridePipelinesDialogText", "You are using override pipelines now. This will erase all the previous modifications made to the default pipeline settings."),
				LOCTEXT("UseOverridePipelinesDialogTitle", "Using Override Pipelines"),
				"InterchangeImportTestPlanUseOverridePipelinesWarning");

			DialogSetupInfo.ConfirmText = LOCTEXT("UseOverridePipelinesDialogOptionConfirm", "OK");
			DialogSetupInfo.bDefaultToSuppressInTheFuture = false;
			DialogSetupInfo.CheckBoxText = LOCTEXT("UseOverridePipelinesDialogCheckBoxText", "Don't show this dialog again");
			DialogSetupInfo.DialogMode = UInterchangeImportTestPlan::GetInterchangeTestPlanWarningDialogMode();
			FSuppressableWarningDialog SuppressableWarningDialog(DialogSetupInfo);
			SuppressableWarningDialog.ShowModal();

			PipelineSettings.ClearPipelines();
			bSendPipelineSettingsChangedEvent = true;
		}
		
		if (PipelineStack.Num() > 0)
		{
			bSendPipelineSettingsChangedEvent = true;
		}

		if (bSendPipelineSettingsChangedEvent)
		{
			BroadcastImportStepChangedEvent(EImportStepDataChangeType::PipelineSettings);
		}
	}
	
	if (PropertyChangedEvent.Property->GetFName() == ImportIntoLevelPropertyName)
	{
		if (ShouldImportIntoLevelChangeRequireMessageBox())
		{
			FSuppressableWarningDialog::FSetupInfo DialogSetupInfo = FSuppressableWarningDialog::FSetupInfo(
				LOCTEXT("ImportIntoLevelDialogMessage", "Import type is changed. This will delete all the modifications made.\nNOTE: This change is irreversible.\nDo you still wish to continue? "),
				LOCTEXT("ImportIntoLevelDialogTitle", "Import Type Changed"),
				"InterchangeImportTestPlanImportIntoLevelWarning");

			DialogSetupInfo.ConfirmText = LOCTEXT("ImportIntoLevelDialogOptionConfirm", "Yes");
			DialogSetupInfo.CancelText =  LOCTEXT("ImportIntoLevelDialogOptionCancel", "No");
			DialogSetupInfo.bDefaultToSuppressInTheFuture = false;
			DialogSetupInfo.CheckBoxText = LOCTEXT("ImportIntoLevelDialogCheckBoxText", "Don't show this dialog again");
			DialogSetupInfo.DialogMode = UInterchangeImportTestPlan::GetInterchangeTestPlanWarningDialogMode();
			FSuppressableWarningDialog SuppressableWarningDialog(DialogSetupInfo);
			FSuppressableWarningDialog::EResult DialogResult = SuppressableWarningDialog.ShowModal();
			if (DialogResult != FSuppressableWarningDialog::Cancel)
			{
				PipelineSettings.ClearPipelines();
				BroadcastImportStepChangedEvent(EImportStepDataChangeType::ImportIntoLevel);
			}
			else
			{
				bImportIntoLevel = !bImportIntoLevel;
			}
		}
		else
		{
			BroadcastImportStepChangedEvent(EImportStepDataChangeType::ImportIntoLevel);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UInterchangeImportTestStepImport::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	static const FName PipelineStackPropertyName = GET_MEMBER_NAME_CHECKED(UInterchangeImportTestStepImport, PipelineStack);
	static const FName SourceFilePropertyName = GET_MEMBER_NAME_CHECKED(UInterchangeImportTestStepImport, SourceFile);

	if (PropertyChangedEvent.GetPropertyName() == PipelineStackPropertyName)
	{
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet 
			|| PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear 
			|| PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayRemove)
		{
			BroadcastImportStepChangedEvent(EImportStepDataChangeType::PipelineSettings);
		}
	}

	const FProperty* ActiveMemberPropertyNode = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
	if (ActiveMemberPropertyNode && (ActiveMemberPropertyNode->GetFName() == SourceFilePropertyName))
	{
		const FString CurrSourceFileExtension = FPaths::GetExtension(SourceFile.FilePath);
		if (!LastSourceFileExtension.IsEmpty() && !CurrSourceFileExtension.IsEmpty()
			&& LastSourceFileExtension != CurrSourceFileExtension
			&& !bUseOverridePipelineStack
			&& !PipelineSettings.CustomPipelines.IsEmpty())
		{
			FSuppressableWarningDialog::FSetupInfo DialogSetupInfo = FSuppressableWarningDialog::FSetupInfo(
				LOCTEXT("SourceFileExtensionChangedDialogText", "Current pipelines might not be compatible with the new source file. Clearing out to restore the defaults."),
				LOCTEXT("SourceFileExtensionChangedDialogTitle", "Source File Extension Changed"),
				"InterchangeImportTestPlanSourceFileExtensionChangedWarning");

			DialogSetupInfo.ConfirmText = LOCTEXT("SourceFileExtensionChangedDialogOptionConfirm", "OK");
			DialogSetupInfo.bDefaultToSuppressInTheFuture = false;
			DialogSetupInfo.CheckBoxText = LOCTEXT("SourceFileExtensionChangedDialogCheckBoxText", "Don't show this dialog again");
			DialogSetupInfo.DialogMode = UInterchangeImportTestPlan::GetInterchangeTestPlanWarningDialogMode();
			FSuppressableWarningDialog SuppressableWarningDialog(DialogSetupInfo);
			SuppressableWarningDialog.ShowModal();

			PipelineSettings.ClearPipelines();
		}
		LastSourceFileExtension = CurrSourceFileExtension;

		BroadcastImportStepChangedEvent(EImportStepDataChangeType::SourceFile);
	}
}

TArray<TObjectPtr<UInterchangePipelineBase>> UInterchangeImportTestStepImport::GetCurrentPipelinesOrDefault() const
{
	TArray<TObjectPtr<UInterchangePipelineBase>> OutPipelines;
	if (bUseOverridePipelineStack)
	{
		OutPipelines = PipelineStack;
	}
	else if (!PipelineSettings.CustomPipelines.IsEmpty())
	{
		OutPipelines = PipelineSettings.CustomPipelines;
	}
	else
	{
		if (SourceFile.FilePath.IsEmpty())
		{
			return OutPipelines;
		}

		UE::Interchange::FScopedSourceData ScopedSourceData(SourceFile.FilePath);
		UE::Interchange::FScopedTranslator ScopedTranslator(ScopedSourceData.GetSourceData());
		if (!ScopedTranslator.GetTranslator())
		{
			UE_LOGF(LogInterchangeTests, Error, "Cannot import file. The source data is not supported. Try enabling the [%ls] extension for Interchange.", *FPaths::GetExtension(ScopedSourceData.GetSourceData()->GetFilename()));
			return OutPipelines;
		}

		const FInterchangeImportSettings& InterchangeImportSettings = FInterchangeProjectSettingsUtils::GetDefaultImportSettings(bImportIntoLevel);
		if (InterchangeImportSettings.PipelineStacks.Num() == 0)
		{
			UE_LOGF(LogInterchangeTests, Error, "Failed to configure pipelines. There is no pipeline stack defined for the content import type.");
			return OutPipelines;
		}

		if (!InterchangeImportSettings.PipelineStacks.Contains(InterchangeImportSettings.DefaultPipelineStack))
		{
			FInterchangeImportSettings& MutableInterchangeImportSettings = FInterchangeProjectSettingsUtils::GetMutableDefaultImportSettings(bImportIntoLevel);
			TArray<FName> Keys;
			MutableInterchangeImportSettings.PipelineStacks.GetKeys(Keys);
			check(Keys.Num() > 0);
			MutableInterchangeImportSettings.DefaultPipelineStack = Keys[0];
		}

		UE::Interchange::FScopedBaseNodeContainer ScopedBaseNodeContainer;

		UInterchangePipelineConfigurationBase* RegisteredPipelineConfiguration = nullptr;
		TSoftClassPtr <UInterchangePipelineConfigurationBase> ImportDialogClass = InterchangeImportSettings.ImportDialogClass;
		if (ImportDialogClass.IsValid())
		{
			UClass* PipelineConfigurationClass = ImportDialogClass.LoadSynchronous();
			if (PipelineConfigurationClass)
			{
				RegisteredPipelineConfiguration = NewObject<UInterchangePipelineConfigurationBase>(GetTransientPackage(), PipelineConfigurationClass, NAME_None, RF_NoFlags);
				if (RegisteredPipelineConfiguration == nullptr)
				{
					UE_LOGF(LogInterchangeTests, Error, "Failed to create a pipeline configuration object.");
					return OutPipelines;
				}
			}
		}

		auto AdjustPipelineSettingForContext = [this, &ScopedBaseNodeContainer](UInterchangePipelineBase* Pipeline)
			{
				EInterchangePipelineContext Context;
				Context = bImportIntoLevel ? EInterchangePipelineContext::SceneImport : EInterchangePipelineContext::AssetImport;
				FInterchangePipelineContextParams ContextParams;
				ContextParams.ContextType = Context;
				ContextParams.BaseNodeContainer = ScopedBaseNodeContainer.GetBaseNodeContainer();
				Pipeline->SetFromReimportOrOverride(bUseOverridePipelineStack);
				Pipeline->AdjustSettingsForContext(ContextParams);
			};

		const bool bCanTranslate = UInterchangeManager::GetInterchangeManager().CanUseTranslator(ScopedTranslator.GetTranslator());
		if (bCanTranslate)
		{
			FScopedSlowTask Progress(2.f, LOCTEXT("TranslatingSourceFile...", "Translating source file..."));
			Progress.MakeDialog();
			Progress.EnterProgressFrame(1.f);

			UInterchangeBaseNodeContainer& BaseNodeContainer = *(ScopedBaseNodeContainer.GetBaseNodeContainer());
			ScopedTranslator.GetTranslator()->SetResultsContainer(NewObject<UInterchangeResultsContainer>(GetTransientPackage(), NAME_None));
			ScopedTranslator.GetTranslator()->Translate(BaseNodeContainer);

			Progress.EnterProgressFrame(1.f);
		}

		const TMap<FName, FInterchangePipelineStack>& DefaultPipelineStacks = InterchangeImportSettings.PipelineStacks;
		TArray<FInterchangeStackInfo> InPipelineStacks;

		for (const TPair<FName, FInterchangePipelineStack>& PipelineStackInfo : DefaultPipelineStacks)
		{
			FName StackName = PipelineStackInfo.Key;
			FInterchangeStackInfo& StackInfo = InPipelineStacks.AddDefaulted_GetRef();
			StackInfo.StackName = StackName;

			const FInterchangePipelineStack& DefPipelineStack = PipelineStackInfo.Value;
			const TArray<FSoftObjectPath>* Pipelines = &DefPipelineStack.Pipelines;

			// If applicable, check to see if a specific pipeline stack is associated with this translator
			for (const FInterchangeTranslatorPipelines& TranslatorPipelines : DefPipelineStack.PerTranslatorPipelines)
			{
				const UClass* TranslatorClass = TranslatorPipelines.Translator.LoadSynchronous();
				if (ScopedTranslator.GetTranslator() && ScopedTranslator.GetTranslator()->IsA(TranslatorClass))
				{
					Pipelines = &TranslatorPipelines.Pipelines;
					break;
				}
			}

			for (int32 PipelineIndex = 0; PipelineIndex < Pipelines->Num(); ++PipelineIndex)
			{
				if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance((*Pipelines)[PipelineIndex]))
				{
					AdjustPipelineSettingForContext(GeneratedPipeline);
					StackInfo.Pipelines.Add(GeneratedPipeline);
				}
			}
		}

		FName DefaultStackName = FInterchangeProjectSettingsUtils::GetDefaultPipelineStackName(bImportIntoLevel, *ScopedSourceData.GetSourceData());
		FInterchangeStackInfo* StackInfoPtr = InPipelineStacks.FindByPredicate([DefaultStackName](const FInterchangeStackInfo& StackInfo)
			{
				return StackInfo.StackName == DefaultStackName;
			});

		if (StackInfoPtr)
		{
			//When we do not show the UI we use the original stack
			OutPipelines = StackInfoPtr->Pipelines;
		}
		else if (InPipelineStacks.Num() > 0)
		{
			//Take the first valid stack
			for (FInterchangeStackInfo& StackInfo : InPipelineStacks)
			{
				if (!StackInfo.Pipelines.IsEmpty())
				{
					OutPipelines = StackInfo.Pipelines;
					break;
				}
			}
		}
		else
		{
			UE_LOGF(LogInterchangeTests, Error, "Interchange Test Plan: Cannot find any valid stack. Could not build shared test plan data.");
		}
	}

	return OutPipelines;
}

void UInterchangeImportTestStepImport::BroadcastImportStepChangedEvent(EImportStepDataChangeType ChangeType)
{
	if (ChangeType == EImportStepDataChangeType::Unknown)
	{
		return;
	}

	FImportStepChangedData ChangedData;
	ChangedData.ChangeType = ChangeType;
	ChangedData.ImportStep = this;
	OnImportTestStepDataChanged.Broadcast(ChangedData);

}
#endif

#undef LOCTEXT_NAMESPACE
