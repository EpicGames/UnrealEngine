// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeImportTestStepReimport.h"

#include "Dialogs/Dialogs.h"
#include "GameFramework/Actor.h"
#include "InterchangeImportTestData.h"
#include "InterchangeImportTestPlan.h"
#include "InterchangeImportTestStepImport.h"
#include "InterchangeManager.h"
#include "InterchangePipelineBase.h"
#include "InterchangeProjectSettings.h"
#include "InterchangeSceneImportAsset.h"
#include "InterchangeTestsLog.h"
#include "JsonObjectConverter.h"
#include "Misc/AutomationTest.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeImportTestStepReimport)

#define LOCTEXT_NAMESPACE "InterchangeImportTestStepReimport"

UInterchangeImportTestStepReimport::UInterchangeImportTestStepReimport()
{
	PipelineSettings.ParentTestStep = this;
}

void UInterchangeImportTestStepReimport::InitializeReimportStepFromImportStep(UInterchangeImportTestStepImport* ImportTestStep)
{
	if (ImportTestStep)
	{
		CachedImportStep = ImportTestStep;
		ImportTestStep->OnImportTestStepDataChanged.AddUObject(this, &UInterchangeImportTestStepReimport::HandleImportPipelineSettingsModified);
		if (LastSourceFileExtension.IsEmpty() && !ImportTestStep->SourceFile.FilePath.IsEmpty())
		{
			LastSourceFileExtension = FPaths::GetExtension(ImportTestStep->SourceFile.FilePath);
		}
	}
}

void UInterchangeImportTestStepReimport::RemoveImportStepPipelineSettingsModifiedDelegate()
{
	if (UInterchangeImportTestStepImport* ImportStep = CachedImportStep.Get())
	{
		ImportStep->OnImportTestStepDataChanged.RemoveAll(this);
	}
}

TTuple<UE::Interchange::FAssetImportResultPtr, UE::Interchange::FSceneImportResultPtr> UInterchangeImportTestStepReimport::StartStep(FInterchangeImportTestData& Data)
{
	// Find the object we wish to reimport
	TArray<UObject*> PotentialObjectsToReimport;
	PotentialObjectsToReimport.Reserve(Data.ResultObjects.Num());

	for (UObject* ResultObject : Data.ResultObjects)
	{
		if (ResultObject->GetClass() == AssetTypeToReimport.Get())
		{
			PotentialObjectsToReimport.Add(ResultObject);
		}
	}

	UObject* AssetToReimport = nullptr;

	if (PotentialObjectsToReimport.Num() == 1)
	{
		AssetToReimport = PotentialObjectsToReimport[0];
	}
	else if (PotentialObjectsToReimport.Num() > 1 && !AssetNameToReimport.IsEmpty())
	{
		for (UObject* Object : PotentialObjectsToReimport)
		{
			if (Object->GetName() == AssetNameToReimport)
			{
				AssetToReimport = Object;
				break;
			}
		}
	}
	
	const FString SourceFilePathString = GetReimportStepSourceFilePathString();
	if (AssetToReimport == nullptr)
	{
		if (SourceFilePathString.IsEmpty())
		{
			return {nullptr, nullptr};
		}
		else
		{
			if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
			{
				CurrentTest->AddInfo(TEXT("Could not find any asset to re-import. Performing an import into the same directory with new file(might get converted to re-import if it has assets with same name)."));
			}
		}
	}

	const bool bIsSceneImport = (bImportIntoLevel 
									&& !SourceFilePathString.IsEmpty()) 
								|| (CachedImportStep.IsValid()
									&& CachedImportStep->bImportIntoLevel
									&& (AssetTypeToReimport.Get() == UInterchangeSceneImportAsset::StaticClass()));

	
	// Start the Interchange import
	UE::Interchange::FScopedSourceData ScopedSourceData(SourceFilePathString);

	FImportAssetParameters Params;
	if (bUseOverridePipelineStack)
	{
		Params.OverridePipelines.Reserve(PipelineStack.Num());
		for (TObjectPtr<UInterchangePipelineBase> Pipeline : PipelineStack)
		{
			Params.OverridePipelines.Add(Pipeline);
		}
	}
	else if (PipelineSettings.CustomPipelines.Num() > 0)
	{
		Params.OverridePipelines.Reserve(PipelineSettings.CustomPipelines.Num());
		for (TObjectPtr<UInterchangePipelineBase> Pipeline : PipelineSettings.CustomPipelines)
		{
			Params.OverridePipelines.Add(Pipeline);
		}
	}
	Params.bIsAutomated = true;
	Params.ImportLevel = CachedImportStep.IsValid() && CachedImportStep->bImportIntoLevel ? Data.TestPlan->GetCurrentLevel() : nullptr;
	Params.ReimportAsset = AssetToReimport;
	
	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	if (bIsSceneImport)
	{
		if (UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(AssetToReimport))
		{
			Params.ImportLevel = nullptr;
		}

		return InterchangeManager.ImportSceneAsync(Data.DestAssetPackagePath, ScopedSourceData.GetSourceData(), Params);
	}
	else
	{
		UE::Interchange::FAssetImportResultPtr AssetImportResult = InterchangeManager.ImportAssetAsync(Data.DestAssetPackagePath, ScopedSourceData.GetSourceData(), Params);
		return {AssetImportResult, nullptr};
	}
}


FTestStepResults UInterchangeImportTestStepReimport::FinishStep(FInterchangeImportTestData& Data, FAutomationTestBase* CurrentTest)
{
	FTestStepResults Results;

	// Run all the tests
	bool bSuccess = PerformTests(Data, CurrentTest);

	// Only keep assets as result objects since the world and its actors are being destroyed
	TArray<UObject*> ResultObjects = MoveTemp(Data.ResultObjects);
	Data.ResultObjects.Reset(ResultObjects.Num());

	for (UObject* Object : ResultObjects)
	{
		if (!Object->IsA<AActor>())
		{
			Data.ResultObjects.Add(Object);
		}
	}

	Results.bTestStepSuccess = bSuccess;
	return Results;
}

FString UInterchangeImportTestStepReimport::GetContextString() const
{
	return FString(TEXT("Reimporting ")) + FPaths::GetCleanFilename(SourceFileToReimport.FilePath);
}

bool UInterchangeImportTestStepReimport::HasScreenshotTest() const
{
	return CachedImportStep.IsValid() && CachedImportStep->bImportIntoLevel && bTakeScreenshot;
}

FInterchangeTestScreenshotParameters UInterchangeImportTestStepReimport::GetScreenshotParameters() const
{
	return ScreenshotParameters;
}

bool UInterchangeImportTestStepReimport::CanEditPipelineSettings() const
{
	if (UInterchangeImportTestStepImport* ImportStep = CachedImportStep.Get())
	{
		if (!ImportStep->CanEditPipelineSettings())
		{
			return false;
		}
	}

	if (bUseOverridePipelineStack)
	{
		if (PipelineStack.IsEmpty())
		{
			return false;
		}

		for (const UInterchangePipelineBase* Pipeline : PipelineStack)
		{
			if (!Pipeline)
			{
				return false;
			}
		}
	}

	return true;
}

void  UInterchangeImportTestStepReimport::EditPipelineSettings()
{
#if WITH_EDITOR
	FString SourceFilePathString  = GetReimportStepSourceFilePathString();
	if (SourceFilePathString.IsEmpty())
	{
		UE_LOGF(LogInterchangeTests, Error, "Import Step doesn't have a valid Source File to get the default pipeline stack.");
		return;
	}

	UE::Interchange::FScopedSourceData ScopedSourceData(SourceFilePathString);
	UE::Interchange::FScopedTranslator ScopedTranslator(ScopedSourceData.GetSourceData());
	if (!ScopedTranslator.GetTranslator())
	{
		UE_LOGF(LogInterchangeTests, Error, "Cannot import file. The source data is not supported. Try enabling the [%ls] extension for Interchange.", *FPaths::GetExtension(ScopedSourceData.GetSourceData()->GetFilename()));
		return;
	}

	if (!CachedImportStep.IsValid())
	{
		UE_LOGF(LogInterchangeTests, Error, "No valid import step found. Make sure this Reimport Step is part of an Interchange Test Plan Asset and is not used independently.");
		return;
	}

	const bool bIsSceneImport = CachedImportStep.IsValid() && CachedImportStep->bImportIntoLevel;
	const FInterchangeImportSettings& InterchangeImportSettings = FInterchangeProjectSettingsUtils::GetDefaultImportSettings(bIsSceneImport);
	if (InterchangeImportSettings.PipelineStacks.Num() == 0)
	{
		UE_LOGF(LogInterchangeTests, Error, "Failed to configure pipelines. There is no pipeline stack defined for the content import type.");
		return;
	}

	if (!InterchangeImportSettings.PipelineStacks.Contains(InterchangeImportSettings.DefaultPipelineStack))
	{
		FInterchangeImportSettings& MutableInterchangeImportSettings = FInterchangeProjectSettingsUtils::GetMutableDefaultImportSettings(bIsSceneImport);
		TArray<FName> Keys;
		MutableInterchangeImportSettings.PipelineStacks.GetKeys(Keys);
		check(Keys.Num() > 0);
		MutableInterchangeImportSettings.DefaultPipelineStack = Keys[0];
	}

	UE::Interchange::FScopedBaseNodeContainer ScopedBaseNodeContainer;

	UInterchangePipelineConfigurationBase* RegisteredPipelineConfiguration = nullptr;
	TSoftClassPtr <UInterchangePipelineConfigurationBase> ImportDialogClass = InterchangeImportSettings.ImportDialogClass;
	if (ImportDialogClass.IsValid())
	{
		UClass* PipelineConfigurationClass = ImportDialogClass.LoadSynchronous();
		if (PipelineConfigurationClass)
		{
			RegisteredPipelineConfiguration = NewObject<UInterchangePipelineConfigurationBase>(GetTransientPackage(), PipelineConfigurationClass, NAME_None, RF_NoFlags);
			if (RegisteredPipelineConfiguration == nullptr)
			{
				UE_LOGF(LogInterchangeTests, Error, "Failed to create a pipeline configuration object.");
				return;
			}
		}
	}

	auto AdjustPipelineSettingForContext = [this, &ScopedBaseNodeContainer, bIsSceneImport](UInterchangePipelineBase* Pipeline)
		{
			EInterchangePipelineContext Context;
			Context = bIsSceneImport ? EInterchangePipelineContext::SceneReimport: EInterchangePipelineContext::AssetReimport;
			FInterchangePipelineContextParams ContextParams;
			ContextParams.ContextType = Context;
			ContextParams.BaseNodeContainer = ScopedBaseNodeContainer.GetBaseNodeContainer();
			Pipeline->SetFromReimportOrOverride(true);
			Pipeline->AdjustSettingsForContext(ContextParams);
		};

	const bool bCanTranslate = UInterchangeManager::GetInterchangeManager().CanUseTranslator(ScopedTranslator.GetTranslator());
	if (bCanTranslate)
	{
		FScopedSlowTask Progress(2.f, LOCTEXT("TranslatingSourceFile...", "Translating source file..."));
		Progress.MakeDialog();
		Progress.EnterProgressFrame(1.f);

		UInterchangeBaseNodeContainer& BaseNodeContainer = *(ScopedBaseNodeContainer.GetBaseNodeContainer());
		ScopedTranslator.GetTranslator()->SetResultsContainer(NewObject<UInterchangeResultsContainer>(GetTransientPackage(), NAME_None));
		ScopedTranslator.GetTranslator()->Translate(BaseNodeContainer);

		Progress.EnterProgressFrame(1.f);
	}

	const TMap<FName, FInterchangePipelineStack>& DefaultPipelineStacks = InterchangeImportSettings.PipelineStacks;
	TArray<FInterchangeStackInfo> InPipelineStacks;
	TArray<UInterchangePipelineBase*> OutPipelines;

	if (bUseOverridePipelineStack)
	{
		if (!PipelineStack.IsEmpty())
		{
			FInterchangeStackInfo& StackInfo = InPipelineStacks.AddDefaulted_GetRef();
			StackInfo.StackName = FName(TEXT("ReimportOverridePipelineStack"));
			for (int32 PipelineClassIndex = 0; PipelineClassIndex < PipelineStack.Num(); ++PipelineClassIndex)
			{
				if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance(PipelineStack[PipelineClassIndex]))
				{
					AdjustPipelineSettingForContext(GeneratedPipeline);
					StackInfo.Pipelines.Add(GeneratedPipeline);
				}
			}

			if (StackInfo.Pipelines.IsEmpty())
			{
				UE_LOGF(LogInterchangeTests, Error, "Failed to configure pipelines. There are no pipelines in the override stack");
				return;
			}
		}
	}
	else if (!PipelineSettings.CustomPipelines.IsEmpty())
	{
		FInterchangeStackInfo& StackInfo = InPipelineStacks.AddDefaulted_GetRef();
		StackInfo.StackName = FName(TEXT("ReimportCustomPipelineStack"));
		for (int32 PipelineIndex = 0; PipelineIndex < PipelineSettings.CustomPipelines.Num(); ++PipelineIndex)
		{
			if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance(PipelineSettings.CustomPipelines[PipelineIndex]))
			{
				AdjustPipelineSettingForContext(GeneratedPipeline);
				StackInfo.Pipelines.Add(GeneratedPipeline);
			}
		}
	}
	else if (CachedImportStep.IsValid())
	{
		// Add the import step's pipeline stack as a starting point if it is using custom as most users would. User can always choose to go back to default stacks.
		FInterchangeStackInfo& StackInfo = InPipelineStacks.AddDefaulted_GetRef();
		StackInfo.StackName = FName(TEXT("ImportStepCustomPipelineStack"));

		TArray<TObjectPtr<UInterchangePipelineBase>> ImportStepPipelines = CachedImportStep->GetCurrentPipelinesOrDefault();
		for (int32 PipelineIndex = 0; PipelineIndex < ImportStepPipelines.Num(); ++PipelineIndex)
		{
			if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance(ImportStepPipelines[PipelineIndex]))
			{
				AdjustPipelineSettingForContext(GeneratedPipeline);
				StackInfo.Pipelines.Add(GeneratedPipeline);
			}
		}
	}

	for (const TPair<FName, FInterchangePipelineStack>& PipelineStackInfo : DefaultPipelineStacks)
	{
		FName StackName = PipelineStackInfo.Key;
		FInterchangeStackInfo& StackInfo = InPipelineStacks.AddDefaulted_GetRef();
		StackInfo.StackName = StackName;

		const FInterchangePipelineStack& DefPipelineStack = PipelineStackInfo.Value;
		const TArray<FSoftObjectPath>* Pipelines = &DefPipelineStack.Pipelines;

		// If applicable, check to see if a specific pipeline stack is associated with this translator
		for (const FInterchangeTranslatorPipelines& TranslatorPipelines : DefPipelineStack.PerTranslatorPipelines)
		{
			const UClass* TranslatorClass = TranslatorPipelines.Translator.LoadSynchronous();
			if (ScopedTranslator.GetTranslator() && ScopedTranslator.GetTranslator()->IsA(TranslatorClass))
			{
				Pipelines = &TranslatorPipelines.Pipelines;
				break;
			}
		}

		for (int32 PipelineIndex = 0; PipelineIndex < Pipelines->Num(); ++PipelineIndex)
		{
			if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance((*Pipelines)[PipelineIndex]))
			{
				AdjustPipelineSettingForContext(GeneratedPipeline);
				StackInfo.Pipelines.Add(GeneratedPipeline);
			}
		}
	}

	//Show the dialog, a plugin should have register this dialog. We use a plugin to be able to use editor code when doing UI
	constexpr bool bIsReimport = true;
	EInterchangePipelineConfigurationDialogResult DialogResult = RegisteredPipelineConfiguration->ScriptedShowTestPlanConfigurationDialog(InPipelineStacks, OutPipelines, ScopedSourceData.GetSourceData(), ScopedTranslator.GetTranslator(), ScopedBaseNodeContainer.GetBaseNodeContainer(), nullptr, bIsSceneImport, bIsReimport);

	if (DialogResult == EInterchangePipelineConfigurationDialogResult::SaveConfig)
	{
		if (bUseOverridePipelineStack)
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("UpdatePipelineSettings", "Update Pipeline Settings"));
			Modify();

			PipelineStack.Empty(OutPipelines.Num());
			for (UInterchangePipelineBase* Pipeline : OutPipelines)
			{
				if (UInterchangePipelineBase* InstancedPipeline = DuplicateObject<UInterchangePipelineBase>(Pipeline, this))
				{
					InstancedPipeline->SetFlags(EObjectFlags::RF_Transactional);
					PipelineStack.Add(InstancedPipeline);
				}
			}

		}
		else
		{
			PipelineSettings.UpdatePipelines(OutPipelines);
		}
	}
#endif	
}

void  UInterchangeImportTestStepReimport::ClearPipelineSettings()
{

#if WITH_EDITOR
	if (bUseOverridePipelineStack)
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("ClearPipelineSettings", "Clear Pipeline Settings"));
		Modify();
		PipelineStack.Empty();
	}
	else
	{
		PipelineSettings.ClearPipelines();
	}
#endif

}

bool UInterchangeImportTestStepReimport::IsUsingOverridePipelines(bool bCheckForValidPipelines) const
{
	if (!bCheckForValidPipelines)
	{
		return bUseOverridePipelineStack;
	}

	if (bUseOverridePipelineStack)
	{
		for (const TObjectPtr<UInterchangePipelineBase>& Pipeline : PipelineStack)
		{
			if (!Pipeline)
			{
				return false;
			}
		}
		return true;
	}

	return false;
}

void UInterchangeImportTestStepReimport::HandleImportPipelineSettingsModified(FImportStepChangedData ChangedData)
{
	if (!bUseOverridePipelineStack && !PipelineSettings.CustomPipelines.IsEmpty())
	{
		FSuppressableWarningDialog::FSetupInfo DialogSetupInfo = FSuppressableWarningDialog::FSetupInfo(
			LOCTEXT("ClearReimportPipelinesDialogText", "You have pipeline settings incompatible with the import step.\nClearing out to restore correct defaults."),
			LOCTEXT("ClearReimportPipelinesDialogTitle", "Incompatible Pipeline Settings"),
			"InterchangeImportTestClearReimportPipelinesWarning");
		
		DialogSetupInfo.ConfirmText = LOCTEXT("ClearReimportPipelinesDialogTextCancel", "OK");
		DialogSetupInfo.bDefaultToSuppressInTheFuture = false;
		DialogSetupInfo.CheckBoxText = LOCTEXT("ClearReimportPipelinesDialogCheckBoxText", "Don't show this dialog again");
		DialogSetupInfo.DialogMode = UInterchangeImportTestPlan::GetInterchangeTestPlanWarningDialogMode();
		FSuppressableWarningDialog SuppressableWarningDialog(DialogSetupInfo);
		SuppressableWarningDialog.ShowModal();

		PipelineSettings.ClearPipelines(ChangedData.ChangeType == EImportStepDataChangeType::ImportIntoLevel);
	}
}

FString UInterchangeImportTestStepReimport::GetReimportStepSourceFilePathString()
{
	if (CachedImportStep.IsValid())
	{
		if (UInterchangeImportTestStepImport* ImportStep = CachedImportStep.Get())
		{
			if (!ImportStep->SourceFile.FilePath.IsEmpty())
			{
				if (!SourceFileToReimport.FilePath.IsEmpty())
				{
					return SourceFileToReimport.FilePath;
				}

				return ImportStep->SourceFile.FilePath;
			}
		}
	}

	return FString();
}

#if WITH_EDITOR
void UInterchangeImportTestStepReimport::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName UseOverridePipelineStackPropertyName = GET_MEMBER_NAME_CHECKED(UInterchangeImportTestStepImport, bUseOverridePipelineStack);

	if (PropertyChangedEvent.Property->GetFName() == UseOverridePipelineStackPropertyName)
	{
		if (bUseOverridePipelineStack && !PipelineSettings.CustomPipelines.IsEmpty())
		{
			FSuppressableWarningDialog::FSetupInfo DialogSetupInfo = FSuppressableWarningDialog::FSetupInfo(
				LOCTEXT("UseOverridePipelinesDialogText", "You are using override pipelines now. This will erase all the previous modifications made to the default pipeline settings."),
				LOCTEXT("UseOverridePipelinesDialogTitle", "Using Override Pipelines"),
				"InterchangeImportTestPlanUseOverridePipelinesWarning");

			DialogSetupInfo.ConfirmText = LOCTEXT("UseOverridePipelinesDialogOptionConfirm", "OK");
			DialogSetupInfo.bDefaultToSuppressInTheFuture = false;
			DialogSetupInfo.CheckBoxText = LOCTEXT("UseOverridePipelinesDialogCheckBoxText", "Don't show this dialog again");
			DialogSetupInfo.DialogMode = UInterchangeImportTestPlan::GetInterchangeTestPlanWarningDialogMode();
			FSuppressableWarningDialog SuppressableWarningDialog(DialogSetupInfo);
			SuppressableWarningDialog.ShowModal();

			PipelineSettings.ClearPipelines();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UInterchangeImportTestStepReimport::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	static const FName SourceFileToReimportPropertyName = GET_MEMBER_NAME_CHECKED(UInterchangeImportTestStepReimport, SourceFileToReimport);

	const FProperty* ActiveMemberPropertyNode = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
	if (ActiveMemberPropertyNode && (ActiveMemberPropertyNode->GetFName() == SourceFileToReimportPropertyName))
	{
		const FString CurrSourceFileExtension = FPaths::GetExtension(SourceFileToReimport.FilePath);
		if (!LastSourceFileExtension.IsEmpty() && !CurrSourceFileExtension.IsEmpty()
			&& LastSourceFileExtension != CurrSourceFileExtension
			&& !bUseOverridePipelineStack
			&& !PipelineSettings.CustomPipelines.IsEmpty())
		{
			FSuppressableWarningDialog::FSetupInfo DialogSetupInfo = FSuppressableWarningDialog::FSetupInfo(
				LOCTEXT("SourceFileExtensionChangedDialogText", "Current pipelines might not be compatible with the new source file. Clearing out to restore the defaults."),
				LOCTEXT("SourceFileExtensionChangedDialogTitle", "Source File Extension Changed"),
				"InterchangeImportTestPlanSourceFileExtensionChangedWarning");

			DialogSetupInfo.ConfirmText = LOCTEXT("SourceFileExtensionChangedDialogOptionConfirm", "OK");
			DialogSetupInfo.bDefaultToSuppressInTheFuture = false;
			DialogSetupInfo.CheckBoxText = LOCTEXT("SourceFileExtensionChangedDialogCheckBoxText", "Don't show this dialog again");
			DialogSetupInfo.DialogMode = UInterchangeImportTestPlan::GetInterchangeTestPlanWarningDialogMode();
			FSuppressableWarningDialog SuppressableWarningDialog(DialogSetupInfo);
			SuppressableWarningDialog.ShowModal();

			PipelineSettings.ClearPipelines();
		}
		LastSourceFileExtension = CurrSourceFileExtension;
	}
	
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif

#undef LOCTEXT_NAMESPACE 