// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeTestFunction.h"
#include "ImportTestFunctions/ImportTestFunctionsBase.h"
#include "UObject/StructOnScope.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeTestFunction)


TSharedPtr<FStructOnScope> FInterchangeTestFunction::ImportParameters()
{
	// Clear the block of memory into which we will import the parameter values from text.
	TSharedPtr<FStructOnScope> ParamData = MakeShared<FStructOnScope>(CheckFunction);

	for (FProperty* ParamProperty : GetParameters())
	{
		ImportParameter(*ParamData, ParamProperty);
	}

	return ParamData;
}


void FInterchangeTestFunction::ImportParameter(FStructOnScope& ParamData, FProperty* ParamProperty)
{
	FName ParamName = ParamProperty->GetFName();
	int32 Offset = ParamProperty->GetOffset_ForUFunction();

	if (const FString* ParamValue = Parameters.Find(ParamName))
	{
		// Read param value from bound params
		ParamProperty->ImportText_Direct(**ParamValue, ParamData.GetStructMemory() + Offset, nullptr, PPF_None);
	}
	else
	{
		// Read param value from default metadata
		FName MetadataKey(FString(TEXT("CPP_Default_")) + ParamName.ToString());
		if (CheckFunction->HasMetaData(MetadataKey))
		{
			FString DefaultValue = CheckFunction->GetMetaData(MetadataKey);
			ParamProperty->ImportText_Direct(*DefaultValue, ParamData.GetStructMemory() + Offset, nullptr, PPF_None);
		}
	}
}


void FInterchangeTestFunction::ExportParameters()
{
#if 0
	check(CheckFunction);
	check(ParamData.IsValid());

	Parameters.Empty(GetNumParameters());

	for (FProperty* ParamProperty : GetParameters())
	{
		FName ParamName = ParamProperty->GetFName();
		int32 Offset = ParamProperty->GetOffset_ForUFunction();

		FString ExportedText;
		ParamProperty->ExportTextItem_Direct(ExportedText, ParamData->GetStructMemory() + Offset, nullptr, nullptr, 0);
		Parameters.Add(ParamName, ExportedText);
	}
#endif
}


TArray<UClass*> FInterchangeTestFunction::GetAvailableAssetClasses()
{
	TArray<UClass*> Classes;

	// Add a null entry to represent "no class"
	Classes.Add(nullptr);

	TArray<UClass*> FunctionClasses;
	GetDerivedClasses(UImportTestFunctionsBase::StaticClass(), FunctionClasses);

	for (UClass* FunctionClass : FunctionClasses)
	{
		UClass* AssociatedClass = FunctionClass->GetDefaultObject<UImportTestFunctionsBase>()->GetAssociatedAssetType();
		if (AssociatedClass != UObject::StaticClass())
		{
			Classes.AddUnique(AssociatedClass);
		}
	}

	return Classes;
}


TArray<UFunction*> FInterchangeTestFunction::GetAvailableFunctions(const UClass* Class)
{
	TArray<UFunction*> Functions;

	// Add a null entry to represent "no function"
	Functions.Add(nullptr);

	if (Class == nullptr)
	{
		return Functions;
	}

	TArray<UClass*> FunctionClasses;
	GetDerivedClasses(UImportTestFunctionsBase::StaticClass(), FunctionClasses);

	for (UClass* FunctionClass : FunctionClasses)
	{
		if (Class->IsChildOf(FunctionClass->GetDefaultObject<UImportTestFunctionsBase>()->GetAssociatedAssetType()))
		{
			ForEachObjectWithOuter(FunctionClass,
				[&Functions](UObject* Object)
				{
					if (UFunction* Function = Cast<UFunction>(Object))
					{
						Functions.Add(Function);
					}
				},
				EGetObjectsFlags::None
			);
		}
	}

	return Functions;
}


bool FInterchangeTestFunction::IsValid(UFunction* Function) const
{
	// Null functions are an allowed special case 
	if (Function == nullptr)
	{
		return true;
	}

	// The first parameter must be either UObject* or TArray<UObject*> that we imported.
	FObjectProperty* FirstParam = CastField<FObjectProperty>(Function->ChildProperties);
	if (FirstParam == nullptr)
	{
		FArrayProperty* FirstParamAsArray = CastField<FArrayProperty>(Function->ChildProperties);
		if (FirstParamAsArray)
		{
			FirstParam = CastField<FObjectProperty>(FirstParamAsArray->Inner);
		}
	}

	// Check that its type corresponds to the expected asset type
	if (FirstParam == nullptr ||
		FirstParam->ArrayDim != 1 ||
		!AssetClass->IsChildOf(FirstParam->PropertyClass))
	{
		return false;
	}

	// Check that there are at least two parameters: the asset pointer, and the return value (which is counted as an out parameter)
	if (Function->NumParms < 2)
	{
		return false;
	}

	// Check that the final parameter is really a return value, and not a function parameter
	if (Function->ReturnValueOffset == MAX_uint16)
	{
		return false;
	}

	// Check that the return value is the correct type
	FStructProperty* ReturnProperty = CastField<FStructProperty>(Function->GetReturnProperty());
	if (ReturnProperty == nullptr ||
		ReturnProperty->Struct != FInterchangeTestFunctionResult::StaticStruct())
	{
		return false;
	}

	return true;
}


bool FInterchangeTestFunction::RequiresArrayOfAssets() const
{
	if (CheckFunction == nullptr)
	{
		return false;
	}

	FArrayProperty* FirstParamAsArray = CastField<FArrayProperty>(CheckFunction->ChildProperties);
	return (FirstParamAsArray != nullptr);
}


UClass* FInterchangeTestFunction::GetExpectedClass() const
{
	// Determine whether the first parameter is a simple UObject*
	FObjectProperty* FirstParam = CastField<FObjectProperty>(CheckFunction->ChildProperties);
	if (FirstParam == nullptr)
	{
		// If not, it must be an array, for this function to be valid
		FArrayProperty* FirstParamAsArray = CastField<FArrayProperty>(CheckFunction->ChildProperties);
		check(FirstParamAsArray);
		FirstParam = CastField<FObjectProperty>(FirstParamAsArray->Inner);
	}

	return FirstParam->PropertyClass;
}


FInterchangeTestFunctionResult FInterchangeTestFunction::Invoke(const TArray<UObject*>& AssetsToTest)
{
	if (CheckFunction == nullptr)
	{
		return FInterchangeTestFunctionResult();
	}

	check(IsValid(CheckFunction));

	// Build a list of the assets of the expected class type, to pass to the function
	TArray<UObject*> AssetsToPass;
	AssetsToPass.Reserve(AssetsToTest.Num());

	auto NameCheck = [](UObject* Asset, const FString& Name)
		{
			if (Asset == nullptr)
			{
				return false;
			}

			if (Name.IsEmpty())
			{
				return true;
			}

			if (Asset->GetClass()->IsChildOf(AActor::StaticClass()))
			{
				//For AActors we are interested about the Labels
				if (Cast<AActor>(Asset)->GetActorLabel() == Name)
				{
					return true;
				}
			}
			else
			{
				if (Asset->GetName() == Name)
				{
					return true;
				}
			}
			return false;
		};

	UClass* ExpectedClass = AssetClass;
	for (UObject* Asset : AssetsToTest)
	{
		if (Asset && Asset->GetClass()->IsChildOf(ExpectedClass))
		{
			// If the asset is of the expected class, add it if the name matches the supplied optional asset name,
			// or if we are not checking a specific name.
			if (NameCheck(Asset, OptionalAssetName))
			{
				AssetsToPass.Add(Asset);
			}
		}
	}

	// Import the text parameters to binary
	TSharedPtr<FStructOnScope> ParamData = ImportParameters();

	if (RequiresArrayOfAssets())
	{
		// The first parameter is the array of UObject* that we imported.
		// Copy its value straight into the parameter data from this function's argument.
		FPlatformMemory::WriteUnaligned<TArray<UObject*>>(ParamData->GetStructMemory(), AssetsToPass);
	}
	else
	{
		// If there was an asset name, but no matches, return with an error
		if (!OptionalAssetName.IsEmpty() && AssetsToPass.Num() == 0)
		{
			FInterchangeTestFunctionResult Result;
			Result.AddError(FString::Printf(
				TEXT("No asset with the name '%s' was found"),
				*OptionalAssetName)
			);

			return Result;
		}

		// If there is not exactly one entry in the array, the result is ambiguous, so return with an error
		if (AssetsToPass.Num() != 1)
		{
			FInterchangeTestFunctionResult Result;
			Result.AddError(FString::Printf(
				TEXT("Could not determine a single '%s' asset to run test on (there were %d possibilities)"),
				*ExpectedClass->GetDisplayNameText().ToString(),
				AssetsToPass.Num())
			);

			return Result;
		}

		// The first parameter is the single UObject* that we imported.
		FPlatformMemory::WriteUnaligned<UObject*>(ParamData->GetStructMemory(), AssetsToPass[0]);
	}

	// Invoke the function now that the parameter binary data has been initialized
	CheckFunction->GetOuter()->ProcessEvent(CheckFunction, ParamData->GetStructMemory());

	// Copy out the return value.
	// This creates a new instance of the return type, which is copy constructed here from the type-punned binary data.
	// The binary data never exists as a "real" instance, so no destructor will ever be called there.
	FInterchangeTestFunctionResult Result = FPlatformMemory::ReadUnaligned<FInterchangeTestFunctionResult>(ParamData->GetStructMemory() + CheckFunction->ReturnValueOffset);
	return Result;
}

