// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeTestPlanPipelineSettings.h"
#include "InterchangeImportTestStepImport.h"
#include "InterchangePipelineBase.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeTestPlanPipelineSettings)

void FInterchangeTestPlanPipelineSettings::UpdatePipelines(const TArray<UInterchangePipelineBase*>& InPipelines, bool bTransactional /*= true*/)
{
	if (ensure(ParentTestStep))
	{
		FScopedTransaction ScopedTransaction(NSLOCTEXT("InterchangeTestPlanPipelineSettings", "UpdatePipelineSettings", "Update Pipeline Settings"));
		ParentTestStep->Modify();
		CustomPipelines.Empty(InPipelines.Num());
		for (const UInterchangePipelineBase* Pipeline : InPipelines)
		{
			CustomPipelines.Emplace(DuplicateObject<UInterchangePipelineBase>(Pipeline, ParentTestStep));
		}
		if (!bTransactional)
		{
			ScopedTransaction.Cancel();
		}
	}
}

void FInterchangeTestPlanPipelineSettings::UpdatePipelines(const TArray<TObjectPtr<UInterchangePipelineBase>>& InPipelines, bool bTransactional /*= true*/)
{
	if (ensure(ParentTestStep))
	{
		FScopedTransaction ScopedTransaction(NSLOCTEXT("InterchangeTestPlanPipelineSettings", "UpdatePipelineSettings", "Update Pipeline Settings"));
		ParentTestStep->Modify();

		CustomPipelines.Empty(InPipelines.Num());
		for (const TObjectPtr<UInterchangePipelineBase>& Pipeline : InPipelines)
		{
			CustomPipelines.Emplace(DuplicateObject<UInterchangePipelineBase>(Pipeline, ParentTestStep));
		}
		if (!bTransactional)
		{
			ScopedTransaction.Cancel();
		}
	}
}

void FInterchangeTestPlanPipelineSettings::ClearPipelines(bool bTransactional /*= true*/)
{
	if (ensure(ParentTestStep) && !CustomPipelines.IsEmpty())
	{
		FScopedTransaction ScopedTransaction(NSLOCTEXT("InterchangeTestPlanPipelineSettings", "ClearPipelineSettings", "Clear Pipeline Settings"));
		ParentTestStep->Modify();
		CustomPipelines.Empty();
		if (!bTransactional)
		{
			ScopedTransaction.Cancel();
		}
	}
}

bool FInterchangeTestPlanPipelineSettings::IsUsingOverridePipelineStack() const
{
	if (UInterchangeImportTestStepImport* StepImport = Cast<UInterchangeImportTestStepImport>(ParentTestStep))
	{
		return StepImport->bUseOverridePipelineStack;
	}
	return false;
}

bool FInterchangeTestPlanPipelineSettings::IsUsingModifiedSettings() const
{
	if (UInterchangeImportTestStepImport* StepImport = Cast<UInterchangeImportTestStepImport>(ParentTestStep))
	{
		if (StepImport->bUseOverridePipelineStack)
		{
			return !StepImport->PipelineStack.IsEmpty();
		}
	}

	return !CustomPipelines.IsEmpty();
}

bool FInterchangeTestPlanPipelineSettings::CanEditPipelineSettings() const
{
	if (ParentTestStep)
	{
		return ParentTestStep->CanEditPipelineSettings();
	}

	return false;
}
