// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeTestsBlueprintFunctionLibrary.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializerWriter.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeTestsBlueprintFunctionLibrary)

namespace UE::Interchange::Tests::Private
{
	class FInterchangeTestsJsonWriter : public TJsonStringWriter<TPrettyJsonPrintPolicy<TCHAR>>
	{
	public:
		explicit FInterchangeTestsJsonWriter(FString* Out) : TJsonStringWriter<TPrettyJsonPrintPolicy<TCHAR>>(Out, 0) 
		{}
	};


	void GetPipelinePropertiesRecursive(FInterchangeTestsJsonWriter& Writer, UInterchangePipelineBase* Pipeline)
	{
		if (!Pipeline)
		{
			return;
		}

		if(UClass* Class = Pipeline->GetClass())
		{
			Writer.WriteValue(TEXT("Class"), Class ? Class->GetName() : TEXT("Null"));
			Writer.WriteValue(TEXT("Name"), Pipeline->GetName());

			for (FProperty* Property = Class->PropertyLink; Property; Property = Property->PropertyLinkNext)
			{
				if (Property->HasAnyPropertyFlags(CPF_Transient))
				{
					continue;
				}

				if (Property->GetFName() == UInterchangePipelineBase::GetPropertiesStatesPropertyName())
				{
					continue;
				}

				if (Property->GetFName() == UInterchangePipelineBase::GetResultsPropertyName())
				{
					continue;
				}

				const FString PropertyName = Property->GetName();
				const FString PropertyType = Property->GetCPPType();

				FObjectProperty* SubObject = CastField<FObjectProperty>(Property);
				UInterchangePipelineBase* SubPipeline = SubObject ? Cast<UInterchangePipelineBase>(SubObject->GetObjectPropertyValue_InContainer(Pipeline)) : nullptr;
				//Add the category name to the key
				FString CategoryName = FString();
	#if WITH_EDITORONLY_DATA
				CategoryName = Property->GetMetaData("Category");
				if (!SubPipeline && CategoryName.IsEmpty())
				{
					//In Editor do not add property with no category
					continue;
				}
				CategoryName.ReplaceCharInline(TEXT('.'), TEXT('_'));
				CategoryName.RemoveSpacesInline();
	#endif
				Writer.WriteObjectStart(PropertyName);
				Writer.WriteValue(TEXT("Type"), PropertyType);
				Writer.WriteValue(TEXT("Category"), CategoryName);
			
				if (FArrayProperty* Array = CastField<FArrayProperty>(Property))
				{
					Writer.WriteArrayStart(TEXT("Value"));
					FScriptArrayHelper_InContainer ArrayHelper(Array, Pipeline);
					for (int32 i = 0; i < ArrayHelper.Num(); i++)
					{
						const int32 PortFlags = 0;
						FString	Buffer;
						Array->Inner->ExportTextItem_Direct(Buffer, ArrayHelper.GetRawPtr(i), ArrayHelper.GetRawPtr(i), Pipeline, PortFlags);
						Writer.WriteValue(Buffer);
					}
					Writer.WriteArrayEnd();
				}
				else if (SubPipeline)
				{
					// Save the settings if the referenced pipeline is a subobject of ours
					if (SubPipeline->IsInOuter(Pipeline))
					{
						UE_LOGF(LogTemp, Display, "Pipeline: %ls -> SubPipeline: %ls", *Pipeline->GetName(), *PropertyName);
						//Go recursive with subObject, like if they are part of the same object
						GetPipelinePropertiesRecursive(Writer, SubPipeline);
					}
				}
				else
				{
					Writer.WriteArrayStart(TEXT("Value"));
					for (int32 Index = 0; Index < Property->ArrayDim; Index++)
					{
						const int32 PortFlags = 0;
						FString	Value;
						Property->ExportText_InContainer(Index, Value, Pipeline, Pipeline, Pipeline, PortFlags);
						Writer.WriteValue(Value);
					}
					Writer.WriteArrayEnd();
				}
				Writer.WriteObjectEnd();
			}
		}
	}
}

FString UInterchangeTestsBlueprintFunctionLibrary::GetPipelinePropertiesAsJSON(UInterchangePipelineBase* Pipeline)
{
	using namespace UE::Interchange::Tests::Private;
	FString ReturnValue;
	FInterchangeTestsJsonWriter JsonWriter(&ReturnValue);
	JsonWriter.WriteObjectStart();
	GetPipelinePropertiesRecursive(JsonWriter, Pipeline);
	JsonWriter.WriteObjectEnd();
	JsonWriter.Close();
    return ReturnValue;
}
