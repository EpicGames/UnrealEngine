// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Math/UnrealMath.h"

namespace InterchangeTestsMathUtilities
{
	template<typename Type, uint32 DecimalPlaces>
	constexpr Type ComputeRoundingFactor()
	{
		if constexpr(DecimalPlaces == 0)
		{
			return Type(1);
		}
		else 
		{
			return Type(10) * ComputeRoundingFactor<Type, DecimalPlaces - 1>();
		}
	}

	template<typename Type, uint32 DecimalPlaces = 3>
	UE::Math::TVector<Type> RoundVectorToDecimalPlaces(const UE::Math::TVector<Type>& Vector)
	{
		Type Factor = ComputeRoundingFactor<Type, DecimalPlaces>();
		UE::Math::TVector<Type> Result(Vector);
		Result.X = (Type)(FMath::RoundToDouble(Result.X * Factor) / Factor);
		Result.Y = (Type)(FMath::RoundToDouble(Result.Y * Factor) / Factor);
		Result.Z = (Type)(FMath::RoundToDouble(Result.Z * Factor) / Factor);
		return Result;
	}

	template<typename Type, uint32 DecimalPlaces = 3>
	UE::Math::TVector4<Type> RoundVector4ToDecimalPlaces(const UE::Math::TVector4<Type>& Vector)
	{
		Type Factor = ComputeRoundingFactor<Type, DecimalPlaces>();
		UE::Math::TVector4<Type> Result(Vector);
		Result.X = (Type)(FMath::RoundToDouble(Result.X * Factor) / Factor);
		Result.Y = (Type)(FMath::RoundToDouble(Result.Y * Factor) / Factor);
		Result.Z = (Type)(FMath::RoundToDouble(Result.Z * Factor) / Factor);
		Result.W = (Type)(FMath::RoundToDouble(Result.W * Factor) / Factor);
		return Result;
	}
}