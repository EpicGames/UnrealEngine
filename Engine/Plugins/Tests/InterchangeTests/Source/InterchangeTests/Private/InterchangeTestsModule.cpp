// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeTestsModule.h"
#include "InterchangeTestsLog.h"
#include "Modules/ModuleManager.h"


#define LOCTEXT_NAMESPACE "InterchangeEditorModule"

DEFINE_LOG_CATEGORY(LogInterchangeTests);


FInterchangeTestsModule& FInterchangeTestsModule::Get()
{
	return FModuleManager::LoadModuleChecked<FInterchangeTestsModule>(INTERCHANGETESTS_MODULE_NAME);
}


bool FInterchangeTestsModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(INTERCHANGETESTS_MODULE_NAME);
}


void FInterchangeTestsModule::StartupModule()
{
}


IMPLEMENT_MODULE(FInterchangeTestsModule, InterchangeTests);


#undef LOCTEXT_NAMESPACE

