// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImportTestFunctionsBase.h"
#include "ActorImportTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

class AActor;
struct FInterchangeTestFunctionResult;


UCLASS(MinimalAPI)
class UActorImportTestFunctions : public UImportTestFunctionsBase
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;

	/** Check whether the expected number of actors are imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckImportedActorCount(const TArray<AActor*>& Actors, int32 ExpectedNumberOfImportedActors);

	/** Check whether the expected number of actors with a given class are imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckActorClassCount(const TArray<AActor*>& Actors, TSubclassOf<AActor> Class, int32 ExpectedNumberOfActors);

	/** Check whether the imported actor has the expected class */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckActorClass(const AActor* Actor, TSubclassOf<AActor> ExpectedClass);

	/** Check whether the generic property (with a given name) in the imported actor has the expected value (or matches with it using regex) */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckActorPropertyValue(const AActor* Actor, const FString& PropertyName, bool bUseRegexToMatchValue, const FString& ExpectedValue);

	/** Check whether the generic property (with a given name) in the imported actor component (with the given name) has the expected value (or matches with it using regex) */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckComponentPropertyValue(const AActor* Actor, const FString& ComponentName, const FString& PropertyName, bool bUseRegexToMatchValue, const FString& ExpectedValue);
};

#undef UE_API
