// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImportTestFunctionsBase.h"
#include "AnimationImportTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

struct FInterchangeTestFunctionResult;


UCLASS(MinimalAPI)
class UAnimationImportTestFunctions : public UImportTestFunctionsBase
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;

	/** Check whether the expected number of anim sequences are imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckImportedAnimSequenceCount(const TArray<UAnimSequence*>& AnimSequences, int32 ExpectedNumberOfImportedAnimSequences);

	/** Check whether the animation length (second) is as expected */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckAnimationLength(UAnimSequence* AnimSequence, float ExpectedAnimationLength);

	/** Check whether the animation frame number is as expected */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckAnimationFrameNumber(UAnimSequence* AnimSequence, int32 ExpectedFrameNumber);

	/** Check whether the given curve key time(sec) for the given curve name has the expected time(sec) */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckCurveKeyTime(UAnimSequence* AnimSequence, const FString& CurveName, int32 KeyIndex, float ExpectedCurveKeyTime);

	/** Check whether the given curve key value for the given curve name has the expected value */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckCurveKeyValue(UAnimSequence* AnimSequence, const FString& CurveName, int32 KeyIndex, float ExpectedCurveKeyValue);

	/** Check whether the given curve key arrive tangent for the given curve name has the expected arrive tangent */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckCurveKeyArriveTangent(UAnimSequence* AnimSequence, const FString& CurveName, int32 KeyIndex, float ExpectedCurveKeyArriveTangent);

	/** Check whether the given curve key arrive tangent weight for the given curve name has the expected arrive tangent weight */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckCurveKeyArriveTangentWeight(UAnimSequence* AnimSequence, const FString& CurveName, int32 KeyIndex, float ExpectedCurveKeyArriveTangentWeight);

	/** Check whether the given curve key leave tangent for the given curve name has the expected leave tangent */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckCurveKeyLeaveTangent(UAnimSequence* AnimSequence, const FString& CurveName, int32 KeyIndex, float ExpectedCurveKeyLeaveTangent);

	/** Check whether the given curve key leave tangent weight for the given curve name has the expected leave tangent weight */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckCurveKeyLeaveTangentWeight(UAnimSequence* AnimSequence, const FString& CurveName, int32 KeyIndex, float ExpectedCurveKeyLeaveTangentWeight);
};

#undef UE_API
