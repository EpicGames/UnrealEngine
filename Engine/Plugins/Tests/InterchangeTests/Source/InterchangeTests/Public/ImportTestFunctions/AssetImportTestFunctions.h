// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImportTestFunctionsBase.h"
#include "AssetImportTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

struct FInterchangeTestFunctionResult;
class UObject;


UCLASS(MinimalAPI)
class UAssetImportTestFunctions : public UImportTestFunctionsBase
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;


	/** Check whether the expected number of metadata for the object are imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckImportedMetadataCount(const UObject* Object, const int32 ExpectedNumberOfMetadataForThisObject);

	/** 
	 * Check whether the expected object metadata key exist.
	 * @Param ExpectedMetadataKey - The object metadata key to pass to the package to retrieve the metadata value
	 */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckMetadataExist(const UObject* Object, const FString& ExpectedMetadataKey);

	/**
	 * Check whether the expected object metadata value is imported.
	 * @Param ExpectedMetadataKey - The object metadata key to pass to the package to retrieve the metadata value
	 * @Param ExpectedMetadataValue - The value to compare the object metadata query with the metadata key
	 */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckMetadataValue(const UObject* Object, const FString& ExpectedMetadataKey, const FString& ExpectedMetadataValue);

	/**
	 * Check whether the object's path has the given string as substring.
	 * @Param ExpectedPathString - The string to look for in the object path.
	 */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckObjectPathHasSubstring(const UObject* Object, const FString& ExpectedPathString);

};

#undef UE_API
