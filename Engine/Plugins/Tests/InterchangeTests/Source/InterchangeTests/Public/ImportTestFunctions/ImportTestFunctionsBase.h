// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImportTestFunctionsBase.generated.h"

#define UE_API INTERCHANGETESTS_API


/**
 * This is the base class for any static class which provides test functions for an asset type.
 * Note that test functions defined in derived classes must be defined as UFUNCTION(Exec)
 * This ensures that default parameters are held as metadata.
 * However these functions are in reality not designed to be called from the console.
 * @todo: create an alternative UFUNCTION tag for exporting default parameters as metadata.
 */
UCLASS(MinimalAPI)
class UImportTestFunctionsBase : public UObject
{
	GENERATED_BODY()

public:

	/** Determine which of the import test functions classes to use for a given asset class type */
	static UE_API UClass* GetImportTestFunctionsClassForAssetType(UClass* AssetClass);

	// UImportTestFunctionsBase interface
	virtual UClass* GetAssociatedAssetType() const { return UObject::StaticClass(); }
};

#undef UE_API
