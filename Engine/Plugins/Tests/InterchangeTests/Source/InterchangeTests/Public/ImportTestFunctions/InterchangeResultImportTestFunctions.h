// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImportTestFunctionsBase.h"
#include "UObject/Package.h"
#include "InterchangeResultImportTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

class UInterchangeResult;
class UInterchangeResultsContainer;

struct FInterchangeTestFunctionResult;


UCLASS(MinimalAPI)
class UInterchangeResultImportTestFunctions : public UImportTestFunctionsBase
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;

	/** Check whether the specified InterchangeResult was emitted during import */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckIfErrorOrWarningWasGenerated(UInterchangeResultsContainer* ResultsContainer, TSubclassOf<UInterchangeResult> ErrorOrWarningClass, const FString& Message);
};

#undef UE_API
