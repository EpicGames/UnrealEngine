// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RealCurve.h"
#include "ImportTestFunctionsBase.h"
#include "LevelSequenceImportTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

class ULevelSequence;
struct FInterchangeTestFunctionResult;


UCLASS(MinimalAPI)
class ULevelSequenceImportTestFunctions : public UImportTestFunctionsBase
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;

	/** Check whether the expected number of level sequences are imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckLevelSequenceCount(const TArray<ULevelSequence*>& LevelSequences, int32 ExpectedNumberOfLevelSequences);

	/** Check whether the level sequence length (second) is as expected */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSequenceLength(const ULevelSequence* LevelSequence, float ExpectedLevelSequenceLength);

	/** Check whether the imported level sequence has the expected number of sections */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSectionCount(const ULevelSequence* LevelSequence, int32 ExpectedNumberOfSections);

	/** Check whether the imported level sequence has the expected interpolation mode for the given section */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSectionInterpolationMode(const ULevelSequence* LevelSequence, int32 SectionIndex, ERichCurveInterpMode ExpectedInterpolationMode);
};

#undef UE_API
