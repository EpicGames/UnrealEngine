// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImportTestFunctionsBase.h"
#include "LevelVariantSetsImportTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

class ULevelVariantSets;

struct FInterchangeTestFunctionResult;


UCLASS(MinimalAPI)
class ULevelVariantSetsImportTestFunctions : public UImportTestFunctionsBase
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;

	/** Check whether the expected number of LevelVariantSets are imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckLevelVariantSetsCount(const TArray<ULevelVariantSets*>& LevelVariantSetsAssets, int32 ExpectedNumberOfLevelVariantSets);

	/** Check whether the imported LevelVariantSets has the expected number of variant sets */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckVariantSetsCount(ULevelVariantSets* LevelVariantSets, int32 ExpectedNumberOfVariantSets);

	/** Check whether the imported LevelVariantSets has the expected number of variants for the given variant set */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckVariantsCount(ULevelVariantSets* LevelVariantSets, const FString& VariantSetName, int32 ExpectedNumberOfVariants);

	/** Check whether the imported LevelVariantSets has the expected number of bindings for the given variant in the given set */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckBindingsCount(ULevelVariantSets* LevelVariantSets, const FString& VariantSetName, const FString& VariantName, int32 ExpectedNumberOfBindings);
};

#undef UE_API
