// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorImportTestFunctions.h"
#include "InterchangeTestFunction.h"
#include "LightImportTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

struct FInterchangeTestFunctionResult;


UCLASS(MinimalAPI)
class ULightImportTestFunctions : public UActorImportTestFunctions
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;

	/** Check whether the light position is correct*/
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckLightPosition(ALight* Light, const FVector& ExpectedLightPosition);

	/** Check whether the light direction is correct*/
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckLightDirection(ALight* Light, const FVector& ExpectedLightDirection);

	/** Check whether the light intensity is correct*/
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckLightIntensity(ALight* Light, float ExpectedLightIntensity);

	/** Check whether the light color is correct*/
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckLightColor(ALight* Light, const FLinearColor& ExpectedLightColor);
};

#undef UE_API
