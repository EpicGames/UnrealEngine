// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "ImportTestFunctionsBase.h"
#include "MaterialImportTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

class UMaterialInterface;

struct FInterchangeTestFunctionResult;


UCLASS(MinimalAPI)
class UMaterialImportTestFunctions : public UImportTestFunctionsBase
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;

	/** Check whether the expected number of materials are imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckImportedMaterialCount(const TArray<UMaterialInterface*>& MaterialInterfaces, int32 ExpectedNumberOfImportedMaterials);

	/** Check whether the expected number of material instances are imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckImportedMaterialInstanceCount(const TArray<UMaterialInterface*>& MaterialInterfaces, int32 ExpectedNumberOfImportedMaterialInstances);

	/** Check whether the imported material has the expected shading model */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckShadingModel(const UMaterialInterface* MaterialInterface, EMaterialShadingModel ExpectedShadingModel);

	/** Check whether the imported material has the expected blend mode */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckBlendMode(const UMaterialInterface* MaterialInterface, EBlendMode ExpectedBlendMode);

	/** Check whether the imported material has the expected two-sided setting */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckIsTwoSided(const UMaterialInterface* MaterialInterface, bool ExpectedIsTwoSided);

	/** Check whether the imported material has the expected opacity mask clip value */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckOpacityMaskClipValue(const UMaterialInterface* MaterialInterface, float ExpectedOpacityMaskClipValue);

	/** Check whether the imported material has the expected scalar parameter value */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckScalarParameter(const UMaterialInterface* MaterialInterface, const FString& ParameterName, float ExpectedParameterValue);

	/** Check whether the imported material has the expected vector parameter value */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckVectorParameter(const UMaterialInterface* MaterialInterface, const FString& ParameterName, FLinearColor ExpectedParameterValue);
};

#undef UE_API
