// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImportTestFunctions/ImportTestFunctionsBase.h"

#include "MaterialXTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

class UImportTestFunctionsBase;

struct FInterchangeTestFunctionResult;

class UMaterialInterface;

UCLASS(MinimalAPI)
class UMaterialXTestFunctions : public UImportTestFunctionsBase
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;

	/** Check whether the expected number of inputs are connected to the MX_StandardSurface material function */
	UFUNCTION(Exec, meta = (DisplayName = "MX: Check Connected Input Count"))
	static UE_API FInterchangeTestFunctionResult CheckConnectedInputCount(const UMaterialInterface* MaterialInterface, int32 ExpectedNumber);

	/** Check whether a specific input of the MX_StandardSurface material function is connected or not */
	UFUNCTION(Exec, meta = (DisplayName = "MX: Check Input Is Connected"))
	static UE_API FInterchangeTestFunctionResult CheckInputConnected(const UMaterialInterface* MaterialInterface, const FString& InputName, bool bIsConnected);
};

#undef UE_API
