// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LightImportTestFunctions.h"
#include "InterchangeTestFunction.h"
#include "PointLightImportTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

struct FInterchangeTestFunctionResult;


UCLASS(MinimalAPI)
class UPointLightImportTestFunctions : public ULightImportTestFunctions
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;

	/** Check whether the light falloff exponent is correct*/
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckLightFalloffExponent(APointLight* Light, float ExpectedLightFalloff);
};

#undef UE_API
