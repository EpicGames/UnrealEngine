// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImportTestFunctionsBase.h"
#include "SkeletalMeshImportTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

struct FInterchangeTestFunctionResult;
class USkeletalMesh;


UCLASS(MinimalAPI)
class USkeletalMeshImportTestFunctions : public UImportTestFunctionsBase
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;

	/** Check whether the expected number of skeletal meshes are imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckImportedSkeletalMeshCount(const TArray<USkeletalMesh*>& Meshes, int32 ExpectedNumberOfImportedSkeletalMeshes);

	/** Check whether the vertex count in the built render data for the given LOD is as expected */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckRenderVertexCount(USkeletalMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfRenderVertices);

	/** Check whether the triangle count in the built render data for the given LOD is as expected */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckRenderTriangleCount(USkeletalMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfRenderTriangles);

	/** Check whether the mesh has the expected number of LODs */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckLodCount(USkeletalMesh* Mesh, int32 ExpectedNumberOfLods);

	/** Check whether the mesh has the expected number of material slots */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckMaterialSlotCount(USkeletalMesh* Mesh, int32 ExpectedNumberOfMaterialSlots);

	/** Check whether the built render data for the given mesh LOD has the expected number of sections */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSectionCount(USkeletalMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfSections);

	/** Check whether the given section in the built render data for the given LOD has the expected number of triangles */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckTriangleCountInSection(USkeletalMesh* Mesh, int32 LodIndex, int32 SectionIndex, int32 ExpectedNumberOfTriangles);

	/** Check whether the mesh has the expected number of UV channels */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckUVChannelCount(USkeletalMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfUVChannels);

	/** Check whether the material name for the given section in the render data for the given LOD is as expected */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSectionMaterialName(USkeletalMesh* Mesh, int32 LodIndex, int32 SectionIndex, const FString& ExpectedMaterialName);

	/** Check whether the imported material slot name for the given section in the render data for the given LOD is as expected */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSectionImportedMaterialSlotName(USkeletalMesh* Mesh, int32 LodIndex, int32 SectionIndex, const FString& ExpectedImportedMaterialSlotName);

	/** Check whether the vertex of the given index is at the expected position */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckVertexIndexPosition(USkeletalMesh* Mesh, int32 LodIndex, int32 VertexIndex, const FVector& ExpectedVertexPosition);

	/** Check whether the vertex of the given index is at the expected normal */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckVertexIndexNormal(USkeletalMesh* Mesh, int32 LodIndex, int32 VertexIndex, const FVector& ExpectedVertexNormal);

	/** Check whether the vertex of the given index is at the expected normal */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckVertexIndexColor(USkeletalMesh* Mesh, int32 LodIndex, int32 VertexIndex, const FColor& ExpectedVertexColor);

	/** Check whether the mesh has the expected number of morph targets */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckMorphTargetCount(USkeletalMesh* Mesh, int32 ExpectedNumberOfMorphTargets);

	/** Check whether the imported morph target name is as expected */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckMorphTargetName(USkeletalMesh* Mesh, int32 MorphTargetIndex, const FString& ExpectedMorphTargetName);

	/** Check whether the mesh has the expected number of bones */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckBoneCount(USkeletalMesh* Mesh, int32 ExpectedNumberOfBones);

	/** Check that the bone of the specified index has the expected position */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckBonePosition(USkeletalMesh* Mesh, int32 BoneIndex, const FVector& ExpectedBonePosition);

	/** Check whether the static mesh expected number of sockets were imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSocketCount(USkeletalMesh* Mesh, int32 ExpectedSocketCount);

	/** Check whether the static mesh given socket index has the expected name */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSocketName(USkeletalMesh* Mesh, int32 SocketIndex, const FString& ExpectedSocketName);

	/** Check whether the static mesh given socket index has the expected location */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSocketLocation(USkeletalMesh* Mesh, int32 SocketIndex, const FVector& ExpectedSocketLocation);

	/** Check that the specified bone name is skinned with the expected number of vertices */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSkinnedVertexCountForBone(USkeletalMesh* Mesh, const FString& BoneName, bool bTestFirstAlternateProfile, int32 ExpectedSkinnedVertexCount);
};

#undef UE_API
