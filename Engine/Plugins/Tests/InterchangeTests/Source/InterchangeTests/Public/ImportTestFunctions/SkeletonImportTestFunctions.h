// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImportTestFunctionsBase.h"
#include "SkeletonImportTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

struct FInterchangeTestFunctionResult;
class USkeleton;


UCLASS(MinimalAPI)
class USkeletonImportTestFunctions : public UImportTestFunctionsBase
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;

	/** Check whether the expected number of skeletal meshes are imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckImportedSkeletonCount(const TArray<USkeleton*>& Skeletons, int32 ExpectedNumberOfImportedSkeletons);
};

#undef UE_API
