// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PointLightImportTestFunctions.h"
#include "InterchangeTestFunction.h"
#include "SpotLightImportTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

struct FInterchangeTestFunctionResult;


UCLASS(MinimalAPI)
class USpotLightImportTestFunctions : public UPointLightImportTestFunctions
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;

	/** Check whether the light inner cone angle is correct*/
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckLightInnerConeAngle(ASpotLight* Light, float ExpectedLightInnerConeAngle);

	/** Check whether the light outer cone angle is correct*/
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckLightOuterConeAngle(ASpotLight* Light, float ExpectedLightOuterConeAngle);
};

#undef UE_API
