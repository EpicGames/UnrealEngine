// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImportTestFunctionsBase.h"
#include "StaticMeshImportTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

struct FMeshBuildSettings;
struct FMeshNaniteSettings;

struct FInterchangeTestFunctionResult;
class UStaticMesh;


UENUM(meta = (Bitmask))
enum class EStaticMeshImportTestGroundTruthBitflags : uint32
{
	CheckLodCountEqual                = (1 << 0),
	CheckVertexCountEqual             = (1 << 1),
	CheckTriangleCountEqual           = (1 << 2),
	CheckUVChannelCountEqual          = (1 << 3),
	CheckCollisionPrimitiveCountEqual = (1 << 4),
	CheckVertexPositionsEqual         = (1 << 5),
	CheckNormalsEqual                 = (1 << 6)
};
ENUM_CLASS_FLAGS(EStaticMeshImportTestGroundTruthBitflags);


UCLASS(MinimalAPI)
class UStaticMeshImportTestFunctions : public UImportTestFunctionsBase
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;

	/** Check whether the expected number of static meshes are imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckImportedStaticMeshCount(const TArray<UStaticMesh*>& Meshes, int32 ExpectedNumberOfImportedStaticMeshes);

	/** Check whether the static mesh has the expected number of LODs */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckLodCount(UStaticMesh* Mesh, int32 ExpectedNumberOfLods);

	/** Check whether the render data for the given mesh LOD has the expected number of sections */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSectionCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfSections);

	/** Check whether the static mesh has the expected number of material slots */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckMaterialSlotCount(UStaticMesh* Mesh, int32 ExpectedNumberOfMaterialSlots);

	/** Check whether the static mesh imported material slot name for the given material index is as expected */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckImportedMaterialSlotName(UStaticMesh* Mesh, int32 MaterialIndex, const FString& ExpectedImportedMaterialSlotName);

	/** Check whether the mesh description has the expected number of polygon groups for the given LOD */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckPolygonGroupCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfPolygonGroups);

	/** Check whether the mesh description material slot name for the given polygon group in the given LOD is as expected */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckPolygonGroupImportedMaterialSlotName(UStaticMesh* Mesh, int32 LodIndex, int32 PolygonGroupIndex, const FString& ExpectedImportedMaterialSlotName);

	/** Check whether the section index in the built render data for the given LOD is referencing the expected material index */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSectionMaterialIndex(UStaticMesh* Mesh, int32 LodIndex, int32 SectionIndex, int32 ExpectedMaterialIndex);

	/** Check whether the section index in the built render data for the given LOD is referencing the expected material */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSectionMaterialName(UStaticMesh* Mesh, int32 LodIndex, int32 SectionIndex, const FString& ExpectedMaterialName);

	/** Check whether the section index in the built render data for the given LOD is referencing the expected imported material slot name*/
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSectionImportedMaterialSlotName(UStaticMesh* Mesh, int32 LodIndex, int32 SectionIndex, const FString& ExpectedImportedMaterialSlotName);

	/** Check whether the mesh description vertex count for the given LOD is as expected */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckVertexCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfVertices);

	/** Check whether the render data vertex count for the given LOD is as expected */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckRenderVertexCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfRenderVertices);

	/** Check whether the mesh description vertex of the given index is at the expected position */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckVertexIndexPosition(UStaticMesh* Mesh, int32 LodIndex, int32 VertexIndex, const FVector& ExpectedVertexPosition);

	/** Check whether the render data vertex index of the given LOD has the expected vertex normal */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckRenderVertexIndexNormal(UStaticMesh* Mesh, int32 LodIndex, int32 VertexIndex, const FVector4f& ExpectedVertexNormal);

	/** Check whether the render data vertex index of the given LOD has the expected vertex color */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckRenderVertexIndexColor(UStaticMesh* Mesh, int32 LodIndex, int32 VertexIndex, const FColor& ExpectedVertexColor);

	/** Check whether the render data contains any vertex colors of the given LOD */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckRenderHasVertexColors(UStaticMesh* Mesh, int32 LodIndex, bool bExpectedHasVertexColors);

	/** Check whether the mesh description has the expected number of triangles for the given LOD */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckTriangleCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedTotalNumberOfTriangles);

	/** Check whether the render data has the expected number of triangles for the given LOD */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckRenderTriangleCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedTotalNumberOfTriangles);

	/** Check whether the mesh description has the expected number of polygons for the given LOD */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckPolygonCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfPolygons);

	/** Check whether the mesh description contains any quads or ngons */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckThatMeshHasQuadsOrNgons(UStaticMesh* Mesh, int32 LodIndex, bool bMeshHasQuadsOrNgons);

	/** Check whether the given mesh description polygon group of the given LOD has the expected number of triangles */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckTriangleCountInPolygonGroup(UStaticMesh* Mesh, int32 LodIndex, int32 PolygonGroupIndex, int32 ExpectedNumberOfTriangles);

	/** Check whether the given mesh description polygon group of the given LOD has the expected number of polygons */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckPolygonCountInPolygonGroup(UStaticMesh* Mesh, int32 LodIndex, int32 PolygonGroupIndex, int32 ExpectedNumberOfPolygons);

	/** Check whether the mesh description has the expected number of UV channels */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckUVChannelCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfUVChannels);

	/** Check whether the render data has the expected number of UV channels */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckRenderUVChannelCount(UStaticMesh* Mesh, int32 LodIndex, int32 ExpectedNumberOfUVChannels);

	/** Check whether the static mesh expected number of simple collision primitives were imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSimpleCollisionPrimitiveCount(UStaticMesh* Mesh, int32 ExpectedSphereElementCount, int32 ExpectedBoxElementCount, int32 ExpectedCapsuleElementCount, int32 ExpectedConvexElementCount, int32 ExpectedTaperedCapsuleElementCount);

	/** Check whether the static mesh expected number of sockets were imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSocketCount(UStaticMesh* Mesh, int32 ExpectedSocketCount);

	/** Check whether the static mesh given socket index has the expected name */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSocketName(UStaticMesh* Mesh, int32 SocketIndex, const FString& ExpectedSocketName);

	/** Check whether the static mesh given socket index has the expected location */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckSocketLocation(UStaticMesh* Mesh, int32 SocketIndex, const FVector& ExpectedSocketLocation);

	/** Check whether the static mesh is equivalent to a ground truth asset */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckAgainstGroundTruth(UStaticMesh* Mesh, TSoftObjectPtr<UStaticMesh> MeshToCompare,
		bool bCheckVertexCountEqual = true,
		bool bCheckTriangleCountEqual = true,
		bool bCheckUVChannelCountEqual = true,
		bool bCheckCollisionPrimitiveCountEqual = true,
		bool bCheckVertexPositionsEqual = true,
		bool bCheckNormalsEqual = true
	);

	/** Check whether the static mesh source model has the expected build settings */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckBuildSettings(UStaticMesh* Mesh, int32 LodIndex, const FMeshBuildSettings& ExpectedBuildSettings);

	/** Check whether the static mesh has the expected Nanite settings */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckNaniteSettings(UStaticMesh* Mesh, const FMeshNaniteSettings& ExpectedNaniteSettings);

	/** Check whether the bounding box size is greater than given value */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckBoundingBoxSizeGreaterThan(UStaticMesh* Mesh, const FVector& GreaterThanSize);

	/** Check whether the bounding box size is less than given value */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckBoundingBoxSizeLessThan(UStaticMesh* Mesh, const FVector& LessThanSize);

	/** Check whether the bounding box size is equal to given value */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckBoundingBoxSize(UStaticMesh* Mesh, const FVector& ExpectedSize);
};

#undef UE_API
