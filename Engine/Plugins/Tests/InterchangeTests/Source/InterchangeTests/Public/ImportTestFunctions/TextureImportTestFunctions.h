// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureDefines.h"
#include "ImportTestFunctionsBase.h"
#include "TextureImportTestFunctions.generated.h"

#define UE_API INTERCHANGETESTS_API

class UTexture;

struct FInterchangeTestFunctionResult;


UCLASS(MinimalAPI)
class UTextureImportTestFunctions : public UImportTestFunctionsBase
{
	GENERATED_BODY()

public:

	// UImportTestFunctionsBase interface
	UE_API virtual UClass* GetAssociatedAssetType() const override;

	/** Check whether the imported textures are of a specific type */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckTextureType(const TArray<UTexture*>& Textures, TSubclassOf<UTexture> ExpectedTextureType, int32 ExpectedNumber);

	/** Check whether the expected number of textures are imported */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckImportedTextureCount(const TArray<UTexture*>& Textures, int32 ExpectedNumberOfImportedTextures);

	/** Check whether the imported texture has the expected filtering mode */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckTextureFilter(const UTexture* Texture, TextureFilter ExpectedTextureFilter);

	/** Check whether the imported texture has the expected addressing mode for X-axis */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckTextureAddressX(const UTexture* Texture, TextureAddress ExpectedTextureAddressX);

	/** Check whether the imported texture has the expected addressing mode for Y-axis */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckTextureAddressY(const UTexture* Texture, TextureAddress ExpectedTextureAddressY);

	/** Check whether the imported texture has the expected addressing mode for Z-axis */
	UFUNCTION(Exec)
	static UE_API FInterchangeTestFunctionResult CheckTextureAddressZ(const UTexture* Texture, TextureAddress ExpectedTextureAddressZ);
};

#undef UE_API
