// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"

class UInterchangeImportTestPlan;
class UInterchangeResultsContainer;


struct FInterchangeImportTestData : public FAssetData
{
public:
	FInterchangeImportTestData()
		: FAssetData()
	{}

	FInterchangeImportTestData(FName InPackageName, FName InPackagePath, FName InAssetName, FTopLevelAssetPath InAssetClassPathName, FAssetDataTagMap InTags = FAssetDataTagMap(), TArrayView<const int32> InChunkIDs = TArrayView<const int32>(), uint32 InPackageFlags = 0)
		: FAssetData(InPackageName, InPackagePath, InAssetName, InAssetClassPathName, InTags, InChunkIDs, InPackageFlags)
	{}
	
	FInterchangeImportTestData(const FString& InLongPackageName, const FString& InObjectPath, FTopLevelAssetPath InAssetClassPathName, FAssetDataTagMap InTags = FAssetDataTagMap(), TArrayView<const int32> InChunkIDs = TArrayView<const int32>(), uint32 InPackageFlags = 0)
		: FAssetData(InLongPackageName, InObjectPath, InAssetClassPathName, InTags, InChunkIDs, InPackageFlags)
	{}

	FInterchangeImportTestData(const UObject* InAsset, FAssetData::ECreationFlags InCreationFlags = ECreationFlags::None)
		: FAssetData(InAsset, InCreationFlags)
	{}
	
	FInterchangeImportTestData(const UObject* InAsset, FAssetData::ECreationFlags InCreationFlags, EAssetRegistryTagsCaller Caller)
		: FAssetData(InAsset, InCreationFlags, Caller)
	{}

	inline FInterchangeImportTestData(const UObject* InAsset, bool bAllowBlueprintClass)
		: FInterchangeImportTestData(InAsset, bAllowBlueprintClass ? ECreationFlags::AllowBlueprintClass : ECreationFlags::None)
	{
	}

	FString DestAssetPackagePath;
	FString DestAssetFilePath;
	UInterchangeImportTestPlan* TestPlan = nullptr;
	UInterchangeResultsContainer* InterchangeResults = nullptr;
	TArray<UObject*> ResultObjects;
	TArray<FAssetData> ImportedAssets;
};
