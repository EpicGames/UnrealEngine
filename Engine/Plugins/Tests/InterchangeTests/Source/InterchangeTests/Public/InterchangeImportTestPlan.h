// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "Dialogs/Dialogs.h"
#include "UObject/ObjectPtr.h"

#include "InterchangeImportTestPlan.generated.h"

#define UE_API INTERCHANGETESTS_API

class UInterchangeImportTestStepBase;
class UInterchangeImportTestStepImport;
class UInterchangeImportTestStepReimport;

/**
* Define a test plan
*/
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, meta=(PrioritizeCategories = "Description Level Definition Import Reimport")) // @TODO: can't find a way to force Run to be below Definition
class UInterchangeImportTestPlan : public UObject
{
	GENERATED_BODY()

public:

	/** Deserialize a test plan from Json */
	UE_API bool ReadFromJson(const FString& Filename);

	/* Serialize a test plan to Json */
	UE_API void WriteToJson(const FString& Filename);

public:
	UE_API UInterchangeImportTestPlan();

	/** Test Description */
	UPROPERTY(EditAnywhere, Category = "Description", Meta = (MultiLine = true))
	FString Description;

	/** Set of steps to perform to carry out this test plan */
	UPROPERTY(Instanced, meta = (DeprecatedProperty, DeprecationMessage = "Use the import and reimport sections to fill up the test steps."))
	TArray<TObjectPtr<UInterchangeImportTestStepBase>> Steps_DEPRECATED;
	
	/**  Level to use for taking Screenshot */
	UPROPERTY(EditAnywhere, Category = Level, meta = (AllowedClasses = "/Script/Engine.World"))
	FSoftObjectPath WorldPath;

	/** File Import Step */
	UPROPERTY(VisibleAnywhere, Instanced, Category = "Import")
	TObjectPtr<UInterchangeImportTestStepImport> ImportStep;

	/** Set of re-imports that will follow the above import step */
	UPROPERTY(EditAnywhere, Instanced, Category = "Reimport", NoClear)
	TArray<TObjectPtr<UInterchangeImportTestStepReimport>> ReimportStack;


public:
	/** Click here to immediately run this single test through the automation framework */
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Automation")
	UE_API void RunThisTest();	
	UE_API bool IsRunningSynchornously() const;

	UE_API void SetupLevelForImport();
	
	UWorld* GetCurrentWorld() const
	{	
		return TransientWorld.Get() ? TransientWorld.Get() : GWorld->GetWorld();
	}

	ULevel* GetCurrentLevel() const 
	{
		return GetCurrentWorld()->GetCurrentLevel();
	}

	UE_API void CleanupLevel();

	UE_API virtual void PostLoad() override;
	UE_API void OnAssetLoaded(UObject* Asset);

#if WITH_EDITOR
	static UE_API FSuppressableWarningDialog::EMode GetInterchangeTestPlanWarningDialogMode();

	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;

	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif
	
private:
	TStrongObjectPtr<UWorld> TransientWorld;

	int32 PrevReimportStackSize = -1;
	bool bRunSynchronously = false;
	bool bChangeObjectOuters = false;
};


namespace UE::Interchange
{
	class FInterchangeImportTestPlanStaticHelpers
	{
	public:
		static UE_API FString GetTestNameFromObjectPathString(const FString& InObjectPathString, bool bAddBeautifiedTestNamePrefix = false);
		static UE_API FString GetBeautifiedTestName();
		static UE_API FString GetInterchangeImportTestRootGameFolder();
	};
}

#undef UE_API
