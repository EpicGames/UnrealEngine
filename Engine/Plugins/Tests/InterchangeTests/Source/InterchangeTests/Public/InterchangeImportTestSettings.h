// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeImportTestSettings.generated.h"


/**
 * Implement settings for the Interchange Import Test
 */
UCLASS(MinimalAPI, config=Engine, defaultconfig)
class UInterchangeImportTestSettings : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, config, Category=Automation)
	FString ImportTestsPath;

	UPROPERTY(EditAnywhere, config, Category=Automation)
	TArray<FString> ImportFiles;
};
