// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeManager.h"
#include "AutomationScreenshotOptions.h"
#include "Containers/EnumAsByte.h"
#include "Engine/EngineBaseTypes.h"
#include "UObject/ObjectPtr.h"
#include "InterchangeImportTestStepBase.generated.h"

#define UE_API INTERCHANGETESTS_API

class UInterchangeImportTestPlan;
class FAutomationTestExecutionInfo;
struct FInterchangeImportTestData;
struct FInterchangeTestFunction;


struct FTestStepResults
{
	bool bTestStepSuccess = true;
	bool bTriggerGC = false;
};

USTRUCT(BlueprintType)
struct FInterchangeTestScreenshotParameters
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Camera)
	bool bAutoFocus = false;

	UPROPERTY(EditAnywhere, Category = Camera, meta=(EditCondition="!bAutoFocus"))
	FVector CameraLocation = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category = Camera, meta = (EditCondition = "!bAutoFocus"))
	FRotator CameraRotation = FRotator::ZeroRotator;

	UPROPERTY(EditAnywhere, Category = Camera, meta = (EditCondition = "bAutoFocus"))
	FString FocusActorName;

	UPROPERTY(EditAnywhere, Category = Camera, meta = (EditCondition = "bAutoFocus"))
	TSubclassOf<AActor> FocusActorClass;

	UPROPERTY(EditAnywhere, Category = Test)
	EComparisonTolerance ComparisonTolerance = EComparisonTolerance::Low;

	UPROPERTY(EditAnywhere, Category = Viewport)
	TEnumAsByte<EViewModeIndex> ViewMode = EViewModeIndex::VMI_Lit;

	UPROPERTY(EditAnywhere, Category = Viewport)
	float WireframeOpacity = 0.2f;

	UPROPERTY(EditAnywhere, Category = Viewport)
	bool bForceGameView = true;
};


UCLASS(MinimalAPI, BlueprintType, EditInlineNew, Abstract, autoExpandCategories = (General, Test))
class UInterchangeImportTestStepBase : public UObject
{
	GENERATED_BODY()

public:
	/** An array of results to check against */
	UPROPERTY(EditAnywhere, Category = General)
	TArray<FInterchangeTestFunction> Tests;
	
	TObjectPtr<UInterchangeImportTestPlan> ParentTestPlan = nullptr;
public:
	virtual TTuple<UE::Interchange::FAssetImportResultPtr, UE::Interchange::FSceneImportResultPtr>
		StartStep(FInterchangeImportTestData& Data) PURE_VIRTUAL(UInterchangeImportTestStepBase::StartStep, return {}; );
	virtual FTestStepResults FinishStep(FInterchangeImportTestData& Data, FAutomationTestBase* CurrentTest) PURE_VIRTUAL(UInterchangeImportTestStepBase::FinishStep, return {}; );
	virtual FString GetContextString() const PURE_VIRTUAL(UInterchangeImportTestStepBase::GetContextString, return {}; );
	virtual bool HasScreenshotTest() const PURE_VIRTUAL(UInterchangeImportTestStepBase::HasScreenshotTest, return false; );
	virtual FInterchangeTestScreenshotParameters GetScreenshotParameters() const PURE_VIRTUAL(UInterchangeImportTestStepBase::GetScreenshotParameters, return FInterchangeTestScreenshotParameters{}; );
	
	virtual bool CanEditPipelineSettings() const PURE_VIRTUAL(UInterchangeImportTestStepBase::CanEditPipelineSettings, return false; );
	virtual void EditPipelineSettings() PURE_VIRTUAL(UInterchangeImportTestStepBase::EditPipelineSettings);
	virtual void ClearPipelineSettings() PURE_VIRTUAL(UInterchangeImportTestStepBase::ClearPipelineSettings);
	virtual bool IsUsingOverridePipelines(bool bCheckForValidPipelines) const PURE_VIRTUAL(UInterchangeImportTestStepBase::IsUsingOverridePipelines, return false; );

	UE_API bool PerformTests(FInterchangeImportTestData& Data, FAutomationTestBase* CurrentTest);
protected:
	UE_API void SaveReloadAssets(FInterchangeImportTestData& Data);
};

#undef UE_API
