// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "InterchangeImportTestPlan.h"
#include "InterchangeImportTestStepBase.h"
#include "InterchangeTestPlanPipelineSettings.h"
#include "UObject/StrongObjectPtr.h"

#include "InterchangeImportTestStepImport.generated.h"

#define UE_API INTERCHANGETESTS_API

class UInterchangeTranslatorBase;
class UInterchangePipelineBase;
class UInterchangeBaseNodeContainer;
class UWorld;

class UInterchangeImportTestStepImport;

enum class EImportStepDataChangeType : uint8
{
	Unknown,
	SourceFile,
	PipelineSettings,
	ImportIntoLevel,
};

struct FImportStepChangedData
{
public:
	EImportStepDataChangeType ChangeType;

	UInterchangeImportTestStepImport* ImportStep = nullptr;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnImportTestStepDataChanged, FImportStepChangedData)


UCLASS(MinimalAPI, BlueprintType, Meta = (DisplayName = "Import a file"))
class UInterchangeImportTestStepImport : public UInterchangeImportTestStepBase
{
	GENERATED_BODY()

public:
	UE_API UInterchangeImportTestStepImport();

	/** The source file to import (path relative to the json script) */
	UPROPERTY(EditAnywhere, Category = General)
	FFilePath SourceFile;

	/** Whether the import should use the override pipeline stack */
	UPROPERTY(EditAnywhere, Category = General)
	bool bUseOverridePipelineStack = false;

	/** The pipeline stack to use when importing (an empty array will use the defaults) */
	UPROPERTY(EditAnywhere, Instanced, Category = General, meta=(DisplayName="Override Pipeline Stack", EditCondition = "bUseOverridePipelineStack", MaxPropertyDepth = 1))
	TArray<TObjectPtr<UInterchangePipelineBase>> PipelineStack;

	UPROPERTY(EditAnywhere, Category = General)
	FInterchangeTestPlanPipelineSettings PipelineSettings;

	/** Whether the destination folder should be emptied prior to import */
	UPROPERTY(EditAnywhere, Category = General)
	bool bEmptyDestinationFolderPriorToImport = true;

	/**  Whether we should use the import into level workflow */
	UPROPERTY(EditAnywhere, Category = General)
	bool bImportIntoLevel = false;

	/**  Whether screenshot would be taken at this stage. */
	UPROPERTY(EditAnywhere, Category = "Screenshot Comparison", meta=(EditCondition = "bImportIntoLevel"))
	bool bTakeScreenshot = false;

	/**  Screen Shot Settings */
	UPROPERTY(EditAnywhere, Category = "Screenshot Comparison", meta = (EditCondition = "bTakeScreenshot && bImportIntoLevel"))
	FInterchangeTestScreenshotParameters ScreenshotParameters;

	FOnImportTestStepDataChanged OnImportTestStepDataChanged;
public:
	// UInterchangeImportTestStepBase interface
	UE_API virtual TTuple<UE::Interchange::FAssetImportResultPtr, UE::Interchange::FSceneImportResultPtr>
		StartStep(FInterchangeImportTestData& Data) override;
	UE_API virtual FTestStepResults FinishStep(FInterchangeImportTestData& Data, FAutomationTestBase* CurrentTest) override;
	UE_API virtual FString GetContextString() const override;
	UE_API virtual bool HasScreenshotTest() const override;
	UE_API virtual FInterchangeTestScreenshotParameters GetScreenshotParameters() const override;

	UE_API virtual bool CanEditPipelineSettings() const override;
	UE_API virtual void EditPipelineSettings() override;
	UE_API virtual void ClearPipelineSettings() override;
	UE_API virtual bool IsUsingOverridePipelines(bool bCheckForValidPipelines) const override;

	UE_API bool ShouldImportIntoLevelChangeRequireMessageBox() const;

	UE_API virtual void PostLoad() override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif
	
	UE_API TArray<TObjectPtr<UInterchangePipelineBase>> GetCurrentPipelinesOrDefault() const;
private:
	UE_API void BroadcastImportStepChangedEvent(EImportStepDataChangeType ChangeType);

private:
	UPROPERTY()
	FString LastSourceFileExtension;

};

#undef UE_API
