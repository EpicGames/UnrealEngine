// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeImportTestStepBase.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectPtr.h"
#include "InterchangeTestPlanPipelineSettings.h"
#include "InterchangeImportTestStepImport.h"
#include "Delegates/DelegateCombinations.h"
#include "InterchangeImportTestStepReimport.generated.h"

#define UE_API INTERCHANGETESTS_API

class UInterchangeImportTestPlan;
class UInterchangePipelineBase;
struct FInterchangeTestFunction;

UCLASS(MinimalAPI, BlueprintType, Meta = (DisplayName = "Reimport a file"), AutoExpandCategories="General")
class UInterchangeImportTestStepReimport : public UInterchangeImportTestStepBase
{
	GENERATED_BODY()

public:
	UE_API UInterchangeImportTestStepReimport();

	/** The source file to import (path relative to the json script). */
	UPROPERTY(EditAnywhere, Category = General)
	FFilePath SourceFileToReimport;

	/** Whether the import should use the override pipeline stack */
	UPROPERTY(EditAnywhere, Category = General)
	bool bUseOverridePipelineStack = false;

	/** The pipeline stack to use when re-importing (an empty array will use the original import pipelines) */
	UPROPERTY(EditAnywhere, Instanced, Category = General, meta = (DisplayName = "Override Pipeline Stack", EditCondition = "bUseOverridePipelineStack", MaxPropertyDepth = 1))
	TArray<TObjectPtr<UInterchangePipelineBase>> PipelineStack;

	/** Pipeline settings that would allow modifying the pipelines as reimport pipelines */
	UPROPERTY(EditAnywhere, Category = General)
	FInterchangeTestPlanPipelineSettings PipelineSettings;

	/** If this is an import into level with new file in the same directory as import */
	UPROPERTY(EditAnywhere, Category = General)
	bool bImportIntoLevel = false;

	/** The type of the asset to reimport. If only one such asset was imported, this is unambiguous. */
	UPROPERTY(EditAnywhere, Category = General, meta=(EditCondition = "!bImportIntoLevel"))
	TSubclassOf<UObject> AssetTypeToReimport;

	/** If there were multiple assets of the above type imported, specify the concrete name here. */
	UPROPERTY(EditAnywhere, Category = General, meta = (EditCondition = "!bImportIntoLevel"))
	FString AssetNameToReimport;

	/** Whether a screenshot should be captured after reimport. */
	UPROPERTY(EditAnywhere, Category = "Screenshot Comparison")
	bool bTakeScreenshot = false;

	/**  Screen Shot Settings */
	UPROPERTY(EditAnywhere, Category = "Screenshot Comparison", meta=(EditCondition="bTakeScreenshot"))
	FInterchangeTestScreenshotParameters ScreenshotParameters;

public:
	UE_API void InitializeReimportStepFromImportStep(UInterchangeImportTestStepImport* ImportTestStep);
	UE_API void RemoveImportStepPipelineSettingsModifiedDelegate();

	// UInterchangeImportTestStepBase interface
	UE_API virtual TTuple<UE::Interchange::FAssetImportResultPtr, UE::Interchange::FSceneImportResultPtr>
		StartStep(FInterchangeImportTestData& Data) override;
	UE_API virtual FTestStepResults FinishStep(FInterchangeImportTestData& Data, FAutomationTestBase* CurrentTest) override;
	UE_API virtual FString GetContextString() const override;
	UE_API virtual bool HasScreenshotTest() const override;
	UE_API virtual FInterchangeTestScreenshotParameters GetScreenshotParameters() const override;

	UE_API virtual bool CanEditPipelineSettings() const override;
	UE_API virtual void EditPipelineSettings() override;
	UE_API virtual void ClearPipelineSettings() override;
	UE_API virtual bool IsUsingOverridePipelines(bool bCheckForValidPipelines) const override;

	UE_API void HandleImportPipelineSettingsModified(FImportStepChangedData ChangedData);

	UE_API FString GetReimportStepSourceFilePathString();

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

private:
	UPROPERTY()
	FString LastSourceFileExtension;

	TWeakObjectPtr<UInterchangeImportTestStepImport> CachedImportStep = nullptr;
};

#undef UE_API
