// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "InterchangeTestFunction.generated.h"

#define UE_API INTERCHANGETESTS_API

class FStructOnScope;


USTRUCT()
struct FInterchangeTestFunctionResult
{
	GENERATED_BODY()

public:
	void AddInfo(FString Message)
	{
		Infos.Add(MoveTemp(Message));
	}

	void AddWarning(FString Message)
	{
		Warnings.Add(MoveTemp(Message));
	}

	void AddError(FString Message)
	{
		Errors.Add(MoveTemp(Message));
	}

	bool IsSuccess() const { return Errors.Num() == 0; }

	const TArray<FString>& GetInfos() const    { return Infos; }
	const TArray<FString>& GetWarnings() const { return Warnings; }
	const TArray<FString>& GetErrors() const   { return Errors; }

private:
	UPROPERTY()
	TArray<FString> Infos;

	UPROPERTY()
	TArray<FString> Warnings;

	UPROPERTY()
	TArray<FString> Errors;
};



USTRUCT()
struct FInterchangeTestFunction
{
	GENERATED_BODY()

public:
	/** Type of the asset being tested */
	UPROPERTY(EditAnywhere, Category = Test)
	TSubclassOf<UObject> AssetClass;

	/** Optional name of the asset to test, if there are various contenders */
	UPROPERTY(EditAnywhere, Category = Test)
	FString OptionalAssetName;

	/** A function to be called to determine whether the result is correct */
	UPROPERTY(EditAnywhere, Category = Test)
	TObjectPtr<UFunction> CheckFunction = nullptr;

	/** Parameters and their bound values as text */
	UPROPERTY(EditAnywhere, Category = Test)
	TMap<FName, FString> Parameters;

public:
	/** Import all parameter data from text to a new binary buffer */
	UE_API TSharedPtr<FStructOnScope> ImportParameters();

	/** Import a single parameter from text to an existing binary buffer */
	UE_API void ImportParameter(FStructOnScope& ParamData, FProperty* Property);

	/** Export parameter data from binary to text */
	UE_API void ExportParameters();

	/** Return a list of the supported asset classes */
	static UE_API TArray<UClass*> GetAvailableAssetClasses();

	/** Return a list of available functions which can act upon an asset of the given class */
	static UE_API TArray<UFunction*> GetAvailableFunctions(const UClass* Class);

	/** Return whether the given function is valid or not */
	UE_API bool IsValid(UFunction* Function) const;

	/** Return whether the function takes an array of assets */
	UE_API bool RequiresArrayOfAssets() const;

	/** Return the class type expected as the first parameter to the function */
	UE_API UClass* GetExpectedClass() const;

	/** Returns the number of parameters which must be bound to this function call */
	int32 GetNumParameters() const { return CheckFunction->NumParms - 2; }

	/** Call the test function bound with the held parameter values */
	UE_API FInterchangeTestFunctionResult Invoke(const TArray<UObject*>& AssetsToTest);

	/**
	 * Provide a readable user-facing way of iterating the function parameters of interest.
	 * Specifically, the first parameter is ignored (it should be a UObject* pointer to the asset which is being tested).
	 * The final parameter is also ignored as it is the return value.
	 */
	class FParameters
	{
	public:

		explicit FParameters(UFunction* InFunction)
			: Function(InFunction)
		{}

		class TConstIterator
		{
		public:
			explicit TConstIterator(const UFunction* InFunction, bool bIsEnd)
			{
				if (InFunction)
				{
					// We have specific requirements for the function:
					// It must have at least one parameter (which must be a UObject* or derived), and it must have a return value.
					// This is enforced in the UI, so we can fatal error here.
					check(InFunction->NumParms > 2);
					check(InFunction->ReturnValueOffset != MAX_uint16);
					check(InFunction->ChildProperties);

					Property = CastField<FProperty>(InFunction->ChildProperties->Next);
					ParamNumber = bIsEnd ? InFunction->NumParms - 1 : 1;
				}
				else
				{
					Property = nullptr;
					ParamNumber = 0;
				}
			}

			TConstIterator& operator++()
			{
				Property = CastField<FProperty>(Property->Next);
				++ParamNumber;
				return *this;
			}

			FProperty* operator*() const
			{
				return Property;
			}

			friend bool operator==(const TConstIterator& A, const TConstIterator& B) { return A.ParamNumber == B.ParamNumber; }
			friend bool operator!=(const TConstIterator& A, const TConstIterator& B) { return A.ParamNumber != B.ParamNumber; }

		private:
			FProperty* Property;
			int32 ParamNumber;
		};

		TConstIterator begin() const { return TConstIterator(Function, false); }
		TConstIterator end() const   { return TConstIterator(Function, true); }

	private:
		UFunction* Function;
	};

	/** Return a proxy object which is used to iterate through function parameters of interest */
	FParameters GetParameters() const { return FParameters(CheckFunction); } 
};

#undef UE_API
