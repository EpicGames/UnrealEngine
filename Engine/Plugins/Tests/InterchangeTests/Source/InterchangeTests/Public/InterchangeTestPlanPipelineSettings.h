// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/DelegateCombinations.h"
#include "UObject/ObjectPtr.h"
#include "InterchangeTestPlanPipelineSettings.generated.h"

class UInterchangeImportTestStepBase;
class UInterchangePipelineBase;

USTRUCT(BlueprintType)
struct FInterchangeTestPlanPipelineSettings
{
	GENERATED_BODY()
public:
	void UpdatePipelines(const TArray<UInterchangePipelineBase*>& InPipelines, bool bTransactional = true);
	void UpdatePipelines(const TArray<TObjectPtr<UInterchangePipelineBase>>& InPipelines, bool bTransactional = true);
	
	void ClearPipelines(bool bTransactional = true);

	INTERCHANGETESTS_API bool IsUsingOverridePipelineStack() const;

	INTERCHANGETESTS_API bool IsUsingModifiedSettings() const;

	INTERCHANGETESTS_API bool CanEditPipelineSettings() const;

public:
	/** Referenced Test Step */
	UPROPERTY(EditAnywhere, Instanced, Category=General)
	TArray<TObjectPtr<UInterchangePipelineBase>> CustomPipelines;

	UPROPERTY()
	TObjectPtr<UInterchangeImportTestStepBase> ParentTestStep;
};