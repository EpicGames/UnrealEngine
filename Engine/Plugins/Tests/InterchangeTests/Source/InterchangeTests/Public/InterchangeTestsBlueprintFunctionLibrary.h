// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "InterchangePipelineBase.h"
#include "InterchangeTestsBlueprintFunctionLibrary.generated.h"

UCLASS(MinimalAPI)
class UInterchangeTestsBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category="Tests")
	static FString GetPipelinePropertiesAsJSON(UInterchangePipelineBase* Pipeline);
};