// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleInterface.h"

#define UE_API INTERCHANGETESTS_API

#define INTERCHANGETESTS_MODULE_NAME TEXT("InterchangeTests")

/**
 * Module for implementing Interchange automation tests
 */
class FInterchangeTestsModule : public IModuleInterface
{
public:
	static UE_API FInterchangeTestsModule& Get();
	static UE_API bool IsAvailable();

private:
	UE_API virtual void StartupModule() override;
};

#undef UE_API
